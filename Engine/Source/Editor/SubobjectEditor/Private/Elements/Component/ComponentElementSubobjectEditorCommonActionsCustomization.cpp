// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Component/ComponentElementSubobjectEditorCommonActionsCustomization.h"
#include "SSubobjectEditor.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "SubobjectData.h"

void FComponentElementSubobjectEditorCommonActionsCustomization::DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements)
{
	if (const TSharedPtr<SSubobjectEditor> Editor = GetSubobjectEditor())
	{
		if (Editor->CanDuplicateComponent())
		{
			Editor->OnDuplicateComponent();

			// OnDuplicateComponent selects the newly duplicated nodes. Convert
			// the current selection to typed element handles so callers can
			// identify what was created.
			for (const FSubobjectDataHandle& Handle : Editor->GetSelectedHandles())
			{
				if (const FSubobjectData* Data = Handle.GetData())
				{
					if (const UActorComponent* Component = Data->GetComponentTemplate())
					{
						if (FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component))
						{
							OutNewElements.Add(MoveTemp(ElementHandle));
						}
					}
				}
			}
			return;
		}
	}

	Super::DuplicateElements(InWorldInterface, InElementHandles, InWorld, InLocationOffset, OutNewElements);
}
