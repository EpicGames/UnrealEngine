// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSubobjectBlueprintEditor.h"
#include "ScopedTransaction.h"
#include "IDocumentation.h"
#include "Widgets/SToolTip.h"
#include "GraphEditorActions.h"
#include "Editor/EditorEngine.h"
#include "ISCSEditorUICustomization.h"	// #TODO_BH Rename this to subobject
#include "SubobjectEditorExtensionContext.h"
#include "ToolMenus.h"
#include "PropertyPath.h"

#include "Styling/SlateIconFinder.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SComponentClassCombo.h"
#include "SPositiveActionButton.h"
#include "Editor/UnrealEdEngine.h"
#include "Subsystems/PanelExtensionSubsystem.h"	// SExtensionPanel
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Dialogs/Dialogs.h"					// FSuppressableWarningDialog
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/ChildActorComponentEditorUtils.h"
#include "ObjectTools.h"						// ThumbnailTools::CacheEmptyThumbnail
#include "K2Node_ComponentBoundEvent.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SSubobjectBlueprintEditor"

extern UNREALED_API UUnrealEdEngine* GUnrealEd;

////////////////////////////////////////////////
// SSubobjectBlueprintEditor

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SSubobjectBlueprintEditor::Construct(const FArguments& InArgs)
{
	ObjectContext = InArgs._ObjectContext;
	PreviewActor = InArgs._PreviewActor;
	OnSelectionUpdated = InArgs._OnSelectionUpdated;
	OnItemDoubleClicked = InArgs._OnItemDoubleClicked;
	OnHighlightPropertyInDetailsView = InArgs._OnHighlightPropertyInDetailsView;
	AllowEditing = InArgs._AllowEditing;
	HideComponentClassCombo = InArgs._HideComponentClassCombo;
	bAllowTreeUpdates = true;
	UICustomization = InArgs._SCSEditorUICustomization;

	InitScopeRow(TEXT("Blueprint Subobject Editor"));

	CreateCommandList();

	// Build the tree widget
	FSlateBrush const* MobilityHeaderBrush = FAppStyle::GetBrush(TEXT("ClassIcon.ComponentMobilityHeaderIcon"));
	
	ConstructTreeWidget();

	// Should only be true when used in the blueprints details panel
	const bool bInlineSearchBarWithButtons = ShowInlineSearchWithButtons();

	TSharedPtr<SWidget> Contents;

	TSharedPtr<SVerticalBox> HeaderBox;
	TSharedPtr<SWidget> SearchBar = SAssignNew(FilterBox, SSearchBox)
		.HintText(!bInlineSearchBarWithButtons ? LOCTEXT("SearchComponentsHint", "Search Components") : LOCTEXT("SearchHint", "Search"))
		.OnTextChanged(this, &SSubobjectBlueprintEditor::OnFilterTextChanged)
		.Visibility(this, &SSubobjectBlueprintEditor::GetComponentsFilterBoxVisibility);

	FMenuBuilder EditBlueprintMenuBuilder = CreateMenuBuilder();

	USubobjectEditorExtensionContext* ExtensionContext = NewObject<USubobjectEditorExtensionContext>();
	ExtensionContext->SubobjectEditor = SharedThis(this);
	ExtensionContext->AddToRoot();

	ButtonBox = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.Padding(0.0f, 0.0f, 4.0f, 0.0f)
	.AutoWidth()
	[
		SNew(SComponentClassCombo)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.AddComponent")))
		.Visibility(this, &SSubobjectBlueprintEditor::GetComponentClassComboButtonVisibility)
		.OnSubobjectClassSelected(this, &SSubobjectBlueprintEditor::PerformComboAddClass)
		.ToolTipText(LOCTEXT("AddComponent_Tooltip", "Adds a new component to this actor"))
		.IsEnabled(true)
		.CustomClassFilters(InArgs._SubobjectClassListFilters)
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.Padding(0.0f, 0.0f, 4.0f, 0.0f)
	.AutoWidth()
	[
		SAssignNew(ExtensionPanel, SExtensionPanel)
		.ExtensionPanelID("SCSEditor.NextToAddComponentButton")
		.ExtensionContext(ExtensionContext)
	]
	
	// horizontal slot index #2 => reserved for BP-editor search bar (see 'ButtonBox' and 'SearchBarHorizontalSlotIndex' usage below)

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.Padding(0.0f, 0.0f, 4.0f, 0.0f)
	.AutoWidth()
	[
		SNew(SPositiveActionButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.ConvertToBlueprint")))
		.Visibility(this, &SSubobjectBlueprintEditor::GetPromoteToBlueprintButtonVisibility)
		.OnClicked(this, &SSubobjectBlueprintEditor::OnPromoteToBlueprintClicked)
		.Icon(FAppStyle::Get().GetBrush("Icons.Blueprints"))
		.ToolTip(IDocumentation::Get()->CreateToolTip(
			TAttribute<FText>(LOCTEXT("PromoteToBluerprintTooltip", "Converts this actor into a reusable Blueprint Class that can have script behavior")),
			nullptr,
			TEXT("Shared/LevelEditor"),
			TEXT("ConvertToBlueprint")))
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SPositiveActionButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.EditBlueprint")))
		.Visibility(this, &SSubobjectBlueprintEditor::GetEditBlueprintButtonVisibility)
		.ToolTipText(LOCTEXT("EditActorBlueprint_Tooltip", "Edit the Blueprint for this Actor"))
		.Icon(FAppStyle::Get().GetBrush("Icons.Blueprints"))
		.MenuContent()
		[
			EditBlueprintMenuBuilder.MakeWidget()
		]
	];

	Contents = SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.VAlign(VAlign_Top)
	.Padding(4.f, 0, 4.f, 4.f)
	[
		SAssignNew(HeaderBox, SVerticalBox)
	]

	+ SVerticalBox::Slot()
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("SCSEditor.Background"))
			.Padding(4.f)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ComponentsPanel")))
			.Visibility(this, &SSubobjectBlueprintEditor::GetComponentsTreeVisibility)
			[
				TreeWidget.ToSharedRef()
			]
		]
		+ SOverlay::Slot()
		[
			SAssignNew(TreeViewOverlay, SBox)
			.Visibility_Lambda([this]()
			{
				const bool bTreeIsVisible = GetComponentsTreeVisibility() == EVisibility::Visible;
				const bool bSupportsOverlay = UICustomization && UICustomization->SupportsTreeViewOverlay(*this);

				if (bTreeIsVisible && bSupportsOverlay)
				{
					const bool bIsDragging = FSlateApplication::Get().IsDragDropping();
					return bIsDragging ? EVisibility::Visible : EVisibility::HitTestInvisible;
				}
				return EVisibility::Collapsed;
			})
			[
				UICustomization ? UICustomization->ConstructTreeViewOverlay(SharedThis(this)) : SNullWidget::NullWidget
			]
		]
	];

	// Only insert the buttons and search bar in the Blueprints version
	if (bInlineSearchBarWithButtons) // Blueprints
	{
		ButtonBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 3.0f)
		[
			SearchBar.ToSharedRef()
		];

		HeaderBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
		[
			ButtonBox.ToSharedRef()
		];
	}

	this->ChildSlot
	[
		Contents.ToSharedRef()
	];

	// Update tree method? 
	UpdateTree();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SSubobjectBlueprintEditor::UpdateTree(bool bRegenerateTreeNodes)
{
	SSubobjectEditor::UpdateTree(bRegenerateTreeNodes);
	if (TreeViewOverlay.IsValid())
	{
		if (UICustomization && UICustomization->SupportsTreeViewOverlay(*this))
		{
			TreeViewOverlay->SetContent(UICustomization->ConstructTreeViewOverlay(SharedThis(this)));
		}
	}
}

void SSubobjectBlueprintEditor::OnDeleteNodes()
{
	const FScopedTransaction Transaction( LOCTEXT("RemoveComponents", "Remove Components") );

	// Invalidate any active component in the visualizer
	GUnrealEd->ComponentVisManager.ClearActiveComponentVis();

	UBlueprint* Blueprint = GetBlueprint();
	
	// Get the current render info for the blueprint. If this is NULL then the blueprint is not currently visualizable (no visible primitive components)
	FThumbnailRenderingInfo* RenderInfo = Blueprint ? GUnrealEd->GetThumbnailManager()->GetRenderingInfo( Blueprint ) : nullptr;

	// A lamda for displaying a confirm message to the user if there is a dynamic delegate bound to the 
	// component they are trying to delete
	auto ConfirmDeleteLambda = [](const FSubobjectData* Data) -> FSuppressableWarningDialog::EResult
	{
		if (ensure(Data))
		{
			FText VarNam = FText::FromName(Data->GetVariableName());
			FText ConfirmDelete = FText::Format(LOCTEXT("ConfirmDeleteDynamicDelegate", "Component \"{0}\" has bound events in use! If you delete it then those nodes will become invalid. Are you sure you want to delete it?"), VarNam);

			// Warn the user that this may result in data loss
			FSuppressableWarningDialog::FSetupInfo Info(ConfirmDelete, LOCTEXT("DeleteComponent", "Delete Component"), "DeleteComponentInUse_Warning");
			Info.ConfirmText = LOCTEXT("ConfirmDeleteDynamicDelegate_Yes", "Yes");
			Info.CancelText = LOCTEXT("ConfirmDeleteDynamicDelegate_No", "No");

			FSuppressableWarningDialog DeleteVariableInUse(Info);

			// If the user selects cancel then return false
			return DeleteVariableInUse.ShowModal();
		}

		return FSuppressableWarningDialog::Cancel;
	};
	
	// Gather the handles of the components that we want to delete
	TArray<FSubobjectDataHandle> HandlesToDelete;
	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = GetSelectedNodes();
	for(const FSubobjectEditorTreeNodePtrType& Node : SelectedNodes)
	{
		check(Node->IsValid());
		
		if(const FSubobjectData* Data = Node->GetDataSource())
		{
			// If this node is in use by Dynamic delegates, then confirm before continuing
			if (FKismetEditorUtilities::PropertyHasBoundEvents(Blueprint, Data->GetVariableName()))
			{
				// The user has decided not to delete the component, stop trying to delete this component
				if (ConfirmDeleteLambda(Data) == FSuppressableWarningDialog::Cancel)
				{
					return;
				}
			}
			HandlesToDelete.Add(Data->GetHandle());
		}
	}

	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		FSubobjectDataHandle HandleToSelect;
		
		int32 NumDeleted = System->DeleteSubobjects(RootNodes[0]->GetDataHandle(), HandlesToDelete, HandleToSelect, GetBlueprint());

		if(NumDeleted > 0)
		{
			if(HandleToSelect.IsValid())
			{
				FSubobjectEditorTreeNodePtrType NodeToSelect = FindSlateNodeForHandle(HandleToSelect);
				if(NodeToSelect.IsValid())
				{
					TreeWidget->SetSelection(NodeToSelect);
				}
			}
			
			UpdateTree();

			// If we had a thumbnail before we deleted any components, check to see if we should clear it
			// If we deleted the final visualizable primitive from the blueprint, GetRenderingInfo should return NULL
			if (Blueprint && RenderInfo)
			{
				FThumbnailRenderingInfo* NewRenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Blueprint);
				if (RenderInfo && !NewRenderInfo)
				{
					// We removed the last visible primitive component, clear the thumbnail
					const FString BPFullName = FString::Printf(TEXT("%s %s"), *Blueprint->GetClass()->GetName(), *Blueprint->GetPathName());
					UPackage* BPPackage = Blueprint->GetOutermost();
					ThumbnailTools::CacheEmptyThumbnail( BPFullName, BPPackage );
				}
			}
			// Do this AFTER marking the Blueprint as modified
			UpdateSelectionFromNodes(TreeWidget->GetSelectedItems());
		}
	}
}

void SSubobjectBlueprintEditor::CopySelectedNodes()
{
	TArray<FSubobjectDataHandle> SelectedHandles = GetSelectedHandles();
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		return System->CopySubobjects(SelectedHandles, GetBlueprint());
	}
}

void SSubobjectBlueprintEditor::OnDuplicateComponent()
{
	const TArray<FSubobjectDataHandle> SelectedNodes = GetSelectedHandles();
	if (SelectedNodes.Num() <= 0)
	{
		return;
	}

	USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
	if (!System)
	{
		return;
	}

	bool ClearedFocus = false;

	// Check if the focus is currently in some sort of textbox
	if (const TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget())
	{
		if (ShouldClearFocusOnDuplicate(FocusedWidget))
		{
			// Force the text box being edited (if any) to commit its text. The duplicate operation may trigger a regeneration of the tree view,
			// releasing all row widgets. If one row was in edit mode (rename/rename on create), it was released before losing the focus and
			// this would prevent the completion of the 'rename' or 'create + give initial name' transaction (occurring on focus lost).
			FSlateApplication::Get().ClearKeyboardFocus();

			ClearedFocus = true;
		}
	}
		
	TUniquePtr<FScopedTransaction> Transaction = MakeUnique<FScopedTransaction>(SelectedNodes.Num() > 1 ? LOCTEXT("DuplicateComponents", "Duplicate Components") : LOCTEXT("DuplicateComponent", "Duplicate Component"));

	TArray<FSubobjectDataHandle> DuplicatedHandles;
	System->DuplicateSubobjects(GetObjectContextHandle(), SelectedNodes, GetBlueprint(), DuplicatedHandles);
	UpdateTree();

	// If nothing was duplicated, no further action is needed
	if (DuplicatedHandles.Num() <= 0)
	{
		return;
	}

	ClearSelection();

	FSubobjectEditorTreeNodePtrType FirstNode = nullptr;

	// Select all duplicated subobjects
	for (const FSubobjectDataHandle& DataHandle : DuplicatedHandles)
	{
		if (const FSubobjectEditorTreeNodePtrType NewNode = FindSlateNodeForHandle(DataHandle))
		{
			if (!FirstNode)
			{
				FirstNode = NewNode;
			}

			SelectNode(NewNode, true);
		}
	}

	// If only one subobject was duplicated, start a rename operation on it
	if (DuplicatedHandles.Num() == 1)
	{
		OnRenameComponent(MoveTemp(Transaction));
	}
	// Otherwise, scroll the first of the duplicated nodes into view in the tree widget
	else if (FirstNode)
	{
		TreeWidget->RequestScrollIntoView(FirstNode);
		if (ClearedFocus)
		{
			// If focus had to be cleared, put it back on the tree widget so further interactions can be made with the new selection
			FSlateApplication::Get().SetKeyboardFocus(TreeWidget, EFocusCause::SetDirectly);
		}
	}
}

bool SSubobjectBlueprintEditor::CanPasteNodes() const
{
	if(!IsEditingAllowed())
	{
		return false;
	}
	
	if(FSubobjectEditorTreeNodePtrType SceneRoot = GetSceneRootNode())
	{
		if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
		{
			return System->CanPasteSubobjects(SceneRoot->GetDataHandle(), GetBlueprint());
		}
	}

	return false;
}

void SSubobjectBlueprintEditor::PasteNodes()
{
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		TArray<FSubobjectDataHandle> OutHandles;
		
		// stop allowing tree updates during paste, otherwise MarkBlueprintAsModified will trigger in the middle of it
		bool bRestoreAllowTreeUpdates = bAllowTreeUpdates;
		bAllowTreeUpdates = false;
		TArray<FSubobjectDataHandle> HandlesToPasteOnto = GetSelectedHandles();
		if(HandlesToPasteOnto.IsEmpty())
		{
			if(FSubobjectEditorTreeNodePtrType RootPtr = GetSceneRootNode())
			{
				if(RootPtr.IsValid())
				{
					HandlesToPasteOnto.Emplace(RootPtr->GetDataHandle());
				}
			}			
		}
		
		System->PasteSubobjects(GetObjectContextHandle(), HandlesToPasteOnto, GetBlueprint(), OutHandles);

		// allow tree updates again
		bAllowTreeUpdates = bRestoreAllowTreeUpdates;
		
		if(OutHandles.Num() > 0)
		{
			// We only want the pasted node(s) to be selected
			TreeWidget->ClearSelection();
			UpdateTree();

			for(const FSubobjectDataHandle& Handle : OutHandles)
			{
				if(FSubobjectEditorTreeNodePtrType SlateNode = FindSlateNodeForHandle(Handle))
				{
					TreeWidget->RequestScrollIntoView(SlateNode);
					TreeWidget->SetItemSelection(SlateNode, true);	
				}
			}
		}
	}
}

void SSubobjectBlueprintEditor::OnAttachToDropAction(FSubobjectEditorTreeNodePtrType DroppedOn, const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs)
{
	// Ask the subsystem to attach the dropped nodes onto the dropped on node
	check(DroppedOn.IsValid());
	check(DroppedNodePtrs.Num() > 0);
	
	USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
	check(System);
	
	const FScopedTransaction TransactionContext(DroppedNodePtrs.Num() > 1 ? LOCTEXT("AttachComponents", "Attach Components") : LOCTEXT("AttachComponent", "Attach Component"));
	TArray<FSubobjectDataHandle> HandlesToMove;
	for(const FSubobjectEditorTreeNodePtrType& DroppedNodePtr : DroppedNodePtrs)
	{
		HandlesToMove.Add(DroppedNodePtr->GetDataHandle());
	}

	FReparentSubobjectParams Params;
	Params.NewParentHandle = DroppedOn->GetDataHandle();
	Params.BlueprintContext =  GetBlueprint();
	Params.ActorPreviewContext = GetActorPreview();

	System->ReparentSubobjects(Params, HandlesToMove);

	check(TreeWidget.IsValid());
	TreeWidget->SetItemExpansion(DroppedOn, true);

	PostDragDropAction(true);
}

void SSubobjectBlueprintEditor::OnDetachFromDropAction(const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs)
{
	check(DroppedNodePtrs.Num() > 0);

	const FScopedTransaction TransactionContext(DroppedNodePtrs.Num() > 1 ? LOCTEXT("DetachComponents", "Detach Components") : LOCTEXT("DetachComponent", "Detach Component"));

	TArray<FSubobjectDataHandle> HandlesToMove;
	Utils::PopulateHandlesArray(DroppedNodePtrs, HandlesToMove);

	// Attach the dropped node to the current scene root node
	FSubobjectEditorTreeNodePtrType SceneRootNodePtr = GetSceneRootNode();
	check(SceneRootNodePtr.IsValid());

	// Ask the subsystem to reparent this object to the scene root
	if (USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		FReparentSubobjectParams Params;
		Params.NewParentHandle = SceneRootNodePtr->GetDataHandle();
		Params.BlueprintContext =  GetBlueprint();
		Params.ActorPreviewContext = GetActorPreview();
		System->ReparentSubobjects(Params, HandlesToMove);
	}

	PostDragDropAction(true);
}

void SSubobjectBlueprintEditor::OnMakeNewRootDropAction(FSubobjectEditorTreeNodePtrType DroppedNodePtr)
{
	// Get the current scene root node
	FSubobjectEditorTreeNodePtrType SceneRootNodePtr = GetSceneRootNode();

	// We cannot handle the drop action if any of these conditions fail on entry.
	if (!ensure(SceneRootNodePtr.IsValid()))
	{
		return;
	}
	if(!ensure(DroppedNodePtr.IsValid()))
	{
		return;
	}
	
	const FScopedTransaction TransactionContext(LOCTEXT("MakeNewSceneRoot", "Make New Scene Root"));
	
	USubobjectDataSubsystem* Subsystem = USubobjectDataSubsystem::Get();
	const bool bSuccess = Subsystem->MakeNewSceneRoot(
		GetObjectContextHandle(),
		 DroppedNodePtr->GetDataHandle(),
		 GetBlueprint());
	
	PostDragDropAction(true);
}

void SSubobjectBlueprintEditor::PostDragDropAction(bool bRegenerateTreeNodes)
{
	GUnrealEd->ComponentVisManager.ClearActiveComponentVis();

	UpdateTree(bRegenerateTreeNodes);

	FBlueprintEditorUtils::PostEditChangeBlueprintActors(GetBlueprint(), true);
}

TSharedPtr<SWidget> SSubobjectBlueprintEditor::BuildSceneRootDropActionMenu(FSubobjectEditorTreeNodePtrType DroppedOntoNodePtr, FSubobjectEditorTreeNodePtrType DroppedNodePtr)
{
	FMenuBuilder MenuBuilder(true, CommandList);

	const FSubobjectData* DroppedNodeData = DroppedNodePtr->GetDataSource();
	const FSubobjectData* DroppedOntoNodeData = DroppedOntoNodePtr->GetDataSource();

	check(DroppedNodeData);
	
	MenuBuilder.BeginSection("SceneRootNodeDropActions", LOCTEXT("SceneRootNodeDropActionContextMenu", "Drop Actions"));
	{
		const FText DroppedVariableNameText = FText::FromName( DroppedNodeData->GetVariableName() );
		
		const FText NodeVariableNameText = FText::FromName( DroppedOntoNodeData->GetVariableName() );

		bool bDroppedInSameBlueprint = true;

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DropActionLabel_AttachToRootNode", "Attach"),
			bDroppedInSameBlueprint 
			? FText::Format( LOCTEXT("DropActionToolTip_AttachToRootNode", "Attach {0} to {1}."), DroppedVariableNameText, NodeVariableNameText )
			: FText::Format( LOCTEXT("DropActionToolTip_AttachToRootNodeFromCopy", "Copy {0} to a new variable and attach it to {1}."), DroppedVariableNameText, NodeVariableNameText ),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSubobjectEditor::OnAttachToDropAction, DroppedOntoNodePtr, DroppedNodePtr),
				FCanExecuteAction()));

		const bool bIsDefaultSceneRoot = DroppedNodeData->IsDefaultSceneRoot();

		FText NewRootNodeText = bIsDefaultSceneRoot
			? FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNodeAndDelete", "Make {0} the new root. The default root will be deleted."), DroppedVariableNameText)
			: FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNode", "Make {0} the new root."), DroppedVariableNameText);

		FText NewRootNodeFromCopyText = bIsDefaultSceneRoot
			? FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNodeFromCopyAndDelete", "Copy {0} to a new variable and make it the new root. The default root will be deleted."), DroppedVariableNameText)
			: FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNodeFromCopy", "Copy {0} to a new variable and make it the new root."), DroppedVariableNameText);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DropActionLabel_MakeNewRootNode", "Make New Root"),
			bDroppedInSameBlueprint ? NewRootNodeText : NewRootNodeFromCopyText,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSubobjectEditor::OnMakeNewRootDropAction, DroppedNodePtr),
				FCanExecuteAction()));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FSubobjectDataHandle SSubobjectBlueprintEditor::AddNewSubobject(const FSubobjectDataHandle& ParentHandle, UClass* NewClass, UObject* AssetOverride, FText& OutFailReason, TUniquePtr<FScopedTransaction> InOngoingTransaction)
{
	FAddNewSubobjectParams Params;
    Params.ParentHandle = ParentHandle;
    Params.NewClass = NewClass;
    Params.AssetOverride = AssetOverride;

    // Make sure to populate the blueprint context of what we are currently editing!
    Params.BlueprintContext = GetBlueprint();
    USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
    check(System);
	
	return System->AddNewSubobject(Params, OutFailReason);
}

void SSubobjectBlueprintEditor::PopulateContextMenuImpl(UToolMenu* InMenu, TArray<FSubobjectEditorTreeNodePtrType>& InSelectedItems, bool bIsChildActorSubtreeNodeSelected)
{
	FToolMenuSection& BlueprintSCSSection = InMenu->AddSection("BlueprintSCS");
	if (InSelectedItems.Num() == 1)
	{
		BlueprintSCSSection.AddSubMenu(
			FName("FindReferenceSubMenu"),
			LOCTEXT("FindReferences_Label", "Find References"),
			LOCTEXT("FindReferences_Tooltip", "Options for finding references to class members"),
			FNewToolMenuChoice(FNewMenuDelegate::CreateStatic(&FGraphEditorCommands::BuildFindReferencesMenu))
		);
	}

	// Create an "Add Event" option in the context menu only if we can edit
	// the currently selected objects
	if (IsEditingAllowed())
	{
		// Collect the classes of all selected objects
		TArray<UClass*> SelectionClasses;
		for (auto NodeIter = InSelectedItems.CreateConstIterator(); NodeIter; ++NodeIter)
		{
			FSubobjectEditorTreeNodePtrType TreeNode = *NodeIter;
			const FSubobjectData* Data = TreeNode->GetDataSource();
			check(Data);
			
			if (const UActorComponent* ComponentTemplate = TreeNode->GetComponentTemplate())
			{
				// If the component is native then we need to ensure it can actually be edited before we display it
				if (!Data->IsNativeComponent() || FComponentEditorUtils::GetPropertyForEditableNativeComponent(ComponentTemplate))
				{
					SelectionClasses.Add(ComponentTemplate->GetClass());
				}
			}
		}

		if (SelectionClasses.Num())
		{
			// Find the common base class of all selected classes
			UClass* SelectedClass = UClass::FindCommonBase(SelectionClasses);
			// Build an event submenu if we can generate events
			UBlueprint* Blueprint = GetBlueprint();
			if (FBlueprintEditorUtils::CanClassGenerateEvents(SelectedClass) && Blueprint != nullptr)
			{
				BlueprintSCSSection.AddSubMenu(
					"AddEventSubMenu",
					LOCTEXT("AddEventSubMenu", "Add Event"),
					LOCTEXT("ActtionsSubMenu_ToolTip", "Add Event"),
					FNewMenuDelegate::CreateStatic(&SSubobjectBlueprintEditor::BuildMenuEventsSection,
						Blueprint,
						SelectedClass,
						FCanExecuteAction::CreateSP(this, &SSubobjectEditor::IsEditingAllowed),
						FGetSelectedObjectsDelegate::CreateSP(this, &SSubobjectEditor::GetSelectedItemsForContextMenu)));
			}
		}
	}

	TArray<UActorComponent*> SelectedComponents;
	for(const FSubobjectEditorTreeNodePtrType& SelectedNodePtr : InSelectedItems)
	{
		check(SelectedNodePtr->IsValid());
		// Get the component template associated with the selected node
		const UActorComponent* ComponentTemplate = SelectedNodePtr->GetComponentTemplate();
		if (ComponentTemplate)
		{
			// #TODO_BH Remove this const cast
			SelectedComponents.Add(const_cast<UActorComponent*>(ComponentTemplate));
		}
	}
	
	// Common menu options added for all component types
    FComponentEditorUtils::FillComponentContextMenuOptions(InMenu, SelectedComponents);

	// For a selection outside of a child actor subtree, we may choose to include additional options
	if (SelectedComponents.Num() == 1 && !bIsChildActorSubtreeNodeSelected)
	{
		// Extra options for a child actor component
		if (UChildActorComponent* SelectedChildActorComponent = Cast<UChildActorComponent>(SelectedComponents[0]))
		{
			// These options will get added only in SCS mode
			FChildActorComponentEditorUtils::FillComponentContextMenuOptions(InMenu, SelectedChildActorComponent);
		}
	}
}

FSubobjectEditorTreeNodePtrType SSubobjectBlueprintEditor::GetSceneRootNode() const
{
	if(RootNodes.Num() == 0)
	{
		return nullptr;
	}
	
	// Loop through all our slate nodes and see if any are marked as scene root
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		FSubobjectDataHandle RootHandle = System->FindSceneRootForSubobject(RootNodes[0]->GetDataHandle());
		if(RootHandle.IsValid())
		{
			return FindSlateNodeForObject(RootHandle.GetSharedDataPtr()->GetObject(), true);			
		}
	}
	return nullptr;
}

FSubobjectEditorTreeNodePtrType SSubobjectBlueprintEditor::FindSlateNodeForObject(const UObject* InObject, bool bIncludeAttachmentComponents) const
{
	if (const UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
	{
		// If the given component instance is not already an archetype object
		if (!ActorComponent->IsTemplate())
		{
			// Get the component owner's class object
			check(ActorComponent->GetOwner() != NULL);
			UClass* OwnerClass = ActorComponent->GetOwner()->GetClass();

			// If the given component is one that's created during Blueprint construction
			if (ActorComponent->IsCreatedByConstructionScript())
			{
				// Check the entire Class hierarchy for the node
				TArray<UBlueprintGeneratedClass*> ParentBPStack;
				UBlueprint::GetBlueprintHierarchyFromClass(OwnerClass, ParentBPStack);

				for (int32 StackIndex = ParentBPStack.Num() - 1; StackIndex >= 0; --StackIndex)
				{
					USimpleConstructionScript* ParentSCS = ParentBPStack[StackIndex] ? ParentBPStack[StackIndex]->SimpleConstructionScript.Get() : nullptr;
					if (ParentSCS)
					{
						// Attempt to locate an SCS node with a variable name that matches the name of the given component
						for (USCS_Node* SCS_Node : ParentSCS->GetAllNodes())
						{
							check(SCS_Node != nullptr);
							if (SCS_Node->GetVariableName() == ActorComponent->GetFName())
							{
								// We found a match; redirect to the component archetype instance that may be associated with a tree node
								ActorComponent = SCS_Node->ComponentTemplate;
								break;
							}
						}
					}
				}
			}
			else
			{
				// Get the class default object
				const AActor* CDO = Cast<AActor>(OwnerClass->GetDefaultObject());
				if (CDO)
				{
					// Iterate over the Components array and attempt to find a component with a matching name
					for (UActorComponent* ComponentTemplate : CDO->GetComponents())
					{
						if (ComponentTemplate && ComponentTemplate->GetFName() == ActorComponent->GetFName())
						{
							// We found a match; redirect to the component archetype instance that may be associated with a tree node
							ActorComponent = ComponentTemplate;
							break;
						}
					}
				}
			}
		}
		InObject = ActorComponent;
	}
	
	return SSubobjectEditor::FindSlateNodeForObject(InObject, bIncludeAttachmentComponents);
}

void SSubobjectBlueprintEditor::BuildMenuEventsSection(FMenuBuilder& Menu, UBlueprint* Blueprint, UClass* SelectedClass, FCanExecuteAction CanExecuteActionDelegate, FGetSelectedObjectsDelegate GetSelectedObjectsDelegate)
{
	// Get Selected Nodes
	TArray<FComponentEventConstructionData> SelectedNodes;
	GetSelectedObjectsDelegate.ExecuteIfBound(SelectedNodes);

	struct FMenuEntry
	{
		FText Label;
		FText ToolTip;
		FUIAction UIAction;
	};

	TArray<FMenuEntry> Actions;
	TArray<FMenuEntry> NodeActions;
	// Build Events entries
	for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(SelectedClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FMulticastDelegateProperty* Property = *PropertyIt;

		// Check for multicast delegates that we can safely assign
		if (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintAssignable))
		{
			FName EventName = Property->GetFName();
			int32 ComponentEventViewEntries = 0;
			// Add View Event Per Component
			for (auto NodeIter = SelectedNodes.CreateConstIterator(); NodeIter; ++NodeIter)
			{
				if(NodeIter->Component.IsValid())
				{
					FName VariableName = NodeIter->VariableName;
					FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>( Blueprint->SkeletonGeneratedClass, VariableName );

					if(VariableProperty && FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, EventName, VariableProperty->GetFName()))
					{
						FMenuEntry NewEntry;
						NewEntry.Label = (SelectedNodes.Num() > 1) ?	FText::Format( LOCTEXT("ViewEvent_ToolTipFor", "{0} for {1}"), FText::FromName(EventName), FText::FromName(VariableName)) : 
																		FText::Format( LOCTEXT("ViewEvent_ToolTip", "{0}"), FText::FromName(EventName));
						NewEntry.UIAction =	FUIAction(FExecuteAction::CreateStatic(&SSubobjectBlueprintEditor::ViewEvent, Blueprint, EventName, *NodeIter), CanExecuteActionDelegate);
						NodeActions.Add(NewEntry);
						ComponentEventViewEntries++;
					}
				}
			}
			if(ComponentEventViewEntries < SelectedNodes.Num())
			{
				// Create menu Add entry
				FMenuEntry NewEntry;
				NewEntry.Label = FText::Format(LOCTEXT("AddEvent_ToolTip", "Add {0}" ), FText::FromName(EventName));
				NewEntry.UIAction =	FUIAction(FExecuteAction::CreateStatic(&SSubobjectBlueprintEditor::CreateEventsForSelection, Blueprint, EventName, GetSelectedObjectsDelegate), CanExecuteActionDelegate);
				Actions.Add(NewEntry);
			}
		}
	}
	// Build Menu Sections
	Menu.BeginSection("AddComponentActions", LOCTEXT("AddEventHeader", "Add Event"));
	for (auto ItemIter = Actions.CreateConstIterator(); ItemIter; ++ItemIter )
	{
		Menu.AddMenuEntry( ItemIter->Label, ItemIter->ToolTip, FSlateIcon(), ItemIter->UIAction );
	}
	Menu.EndSection();
	Menu.BeginSection("ViewComponentActions", LOCTEXT("ViewEventHeader", "View Existing Events"));
	for (auto ItemIter = NodeActions.CreateConstIterator(); ItemIter; ++ItemIter )
	{
		Menu.AddMenuEntry( ItemIter->Label, ItemIter->ToolTip, FSlateIcon(), ItemIter->UIAction );
	}
	Menu.EndSection();
}

void SSubobjectBlueprintEditor::HighlightTreeNode(FName TreeNodeName, const FPropertyPath& Property)
{
	// If there are no nodes then there is nothing to do!
	if (RootNodes.Num() == 0)
	{
		return;
	}

	// Find the tree node that matches up with the given property FName
	FSubobjectEditorTreeNodePtrType FoundNode = nullptr;

	TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
	DepthFirstTraversal(RootNodes[0], VisitedNodes,
		[&FoundNode, TreeNodeName](
		const FSubobjectEditorTreeNodePtrType& CurNodePtr)
		{
			if (CurNodePtr->GetDataHandle().IsValid())
			{
				if (CurNodePtr->GetDataHandle().GetSharedDataPtr()->GetVariableName() == TreeNodeName)
				{
					FoundNode = CurNodePtr;
				}
			}
		}
	);


	// If it was found, then select it and broadcast the delegate to let the diff panel know
	if (FoundNode)
	{
		SelectNode(FoundNode->AsShared(), false);

		if (Property != FPropertyPath())
		{
			// Invoke the delegate to highlight the property
			OnHighlightPropertyInDetailsView.ExecuteIfBound(Property);
		}

		return;
	}

	ClearSelection();
}

void SSubobjectBlueprintEditor::CreateEventsForSelection(UBlueprint* Blueprint, FName EventName, FGetSelectedObjectsDelegate GetSelectedObjectsDelegate)
{
	if (EventName != NAME_None)
	{
		TArray<FComponentEventConstructionData> SelectedNodes;
		GetSelectedObjectsDelegate.ExecuteIfBound(SelectedNodes);

		for (auto SelectionIter = SelectedNodes.CreateConstIterator(); SelectionIter; ++SelectionIter)
		{
			ConstructEvent(Blueprint, EventName, *SelectionIter);
		}
	}
}

void SSubobjectBlueprintEditor::ConstructEvent(UBlueprint* Blueprint, const FName EventName, const FComponentEventConstructionData EventData)
{
	// Find the corresponding variable property in the Blueprint
	FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>(Blueprint->SkeletonGeneratedClass, EventData.VariableName );

	if (VariableProperty)
	{
		if (!FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, EventName, VariableProperty->GetFName()))
		{
			FKismetEditorUtilities::CreateNewBoundEventForComponent(EventData.Component.Get(), EventName, Blueprint, VariableProperty);
		}
	}
}

void SSubobjectBlueprintEditor::ViewEvent(UBlueprint* Blueprint, const FName EventName, const FComponentEventConstructionData EventData)
{
	// Find the corresponding variable property in the Blueprint
	FObjectProperty* VariableProperty = FindFProperty<FObjectProperty>(Blueprint->SkeletonGeneratedClass, EventData.VariableName);

	if (VariableProperty)
	{
		const UK2Node_ComponentBoundEvent* ExistingNode = FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, EventName, VariableProperty->GetFName());
		if (ExistingNode)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingNode);
		}
	}
}

#undef LOCTEXT_NAMESPACE
