// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSubobjectEditor.h"

#include "DataStorage/Scope/EditorDataScope.h"
#include "DataStorage/Features.h"
#include "Editor.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "ClassIconFinder.h"
#include "ScopedTransaction.h"
#include "ComponentAssetBroker.h"		// FComponentAssetBrokerage
#include "TutorialMetaData.h"
#include "IDocumentation.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/ChildActorComponentEditorUtils.h"
#include "GraphEditorActions.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"
#include "ISCSEditorUICustomization.h"	// #TODO_BH Rename this to subobject
#include "SubobjectEditorMenuContext.h"
#include "Toolkits/ToolkitManager.h"
#include "BlueprintEditor.h"
#include "ToolMenus.h"
#include "Components/ChildActorComponent.h"
#include "Components/PrimitiveComponent.h"

#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "SourceCodeNavigation.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Selection.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SComponentClassCombo.h"
#include "SPositiveActionButton.h"
#include "CreateBlueprintFromActorDialog.h"
#include "ClassViewerFilter.h"
#include "GameProjectGenerationModule.h"	// Adding new component classes
#include "AddToProjectConfig.h"
#include "FeaturedClasses.inl"
#include "Settings/EditorStyleSettings.h"

#include "Subsystems/PanelExtensionSubsystem.h"	// SExtensionPanel
#include "Subsystems/AssetEditorSubsystem.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "BPVariableDragDropAction.h"

#include "DragAndDrop/AssetDragDropOp.h"
#include "K2Node_Variable.h"		// Nodes for FSubobjectRowDragDropOp
#include "K2Node_VariableGet.h"
#include "AssetSelection.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "Misc/FeedbackContext.h"

extern UNREALED_API UEditorEngine* GEditor;

#define LOCTEXT_NAMESPACE "SSubobjectEditor"

DEFINE_LOG_CATEGORY_STATIC(LogSubobjectEditor, Log, All);

static const FName SubobjectTree_ColumnName_ComponentClass("ComponentClass");
static const FName SubobjectTree_ColumnName_Asset("Asset");
static const FName SubobjectTree_ColumnName_Mobility("Mobility"); 
static const FName SubobjectTree_ContextMenuName("Kismet.SubobjectEditorContextMenu");
static const FName SubobjectEditor_ClearFocusOnDuplicate("SubobjectEditor_ClearFocusOnDuplicate");

///////////////////////////////////////////////////////////
// Util Functions
namespace SubobjectEditorUtils
{	
	static bool ContainsChildActorSubtreeNode(const TArray<FSubobjectEditorTreeNodePtrType>& InNodePtrs)
	{
		for (FSubobjectEditorTreeNodePtrType NodePtr : InNodePtrs)
		{
			const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
			if (Data && Data->IsChildActorSubtreeObject())
			{
				return true;
			}
		}
	
		return false;
	}
	
	/** Returns whether child actor tree view expansion is enabled in project settings */
	static bool IsChildActorTreeViewExpansionEnabled()
	{
		const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
		return EditorProjectSettings->bEnableChildActorExpansionInTreeView;
	}

	static void OnSetChildActorTreeViewVisualizationMode(const UChildActorComponent* InChildActorComponent, EChildActorComponentTreeViewVisualizationMode InMode, TWeakPtr<SSubobjectEditor> InWeakSubobjectEditorPtr)
	{
		if (!InChildActorComponent)
		{
			return;
		}

		// #TODO_BH Get rid of const cast
		const_cast<UChildActorComponent*>(InChildActorComponent)->SetEditorTreeViewVisualizationMode(InMode);

		TSharedPtr<SSubobjectEditor> SubobjectEditor = InWeakSubobjectEditorPtr.Pin();
		if (SubobjectEditor.IsValid())
		{
			SubobjectEditor->UpdateTree();
		}
	}
	
	static void FillChildActorContextMenuOptions(UToolMenu* Menu, const FSubobjectEditorTreeNodePtrType InNodePtr)
	{
		if (!IsChildActorTreeViewExpansionEnabled())
		{
			return;
		}

		if (!InNodePtr || !InNodePtr->IsChildSubtreeNode())
		{
			return;
		}

		check(InNodePtr.IsValid());

		const UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(InNodePtr->GetComponentTemplate());
		if (!ChildActorComponent)
		{
			return;
		}

		FToolMenuSection& Section = Menu->AddSection("ChildActor", LOCTEXT("ChildActorHeading", "Child Actor"));
		{
			TWeakPtr<SSubobjectEditor> WeakEditorPtr;
			if (USubobjectEditorMenuContext* MenuContext = Menu->FindContext<USubobjectEditorMenuContext>())
			{
				WeakEditorPtr = MenuContext->SubobjectEditor;
			}

			Section.AddMenuEntry(
				"SetChildActorOnlyMode",
				LOCTEXT("SetChildActorOnlyMode_Label", "Switch to Child Actor Only Mode"),
				LOCTEXT("SetChildActorOnlyMode_ToolTip", "Visualize this child actor's template/instance subtree in place of its parent component node."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(
						&SubobjectEditorUtils::OnSetChildActorTreeViewVisualizationMode,
						ChildActorComponent,
						EChildActorComponentTreeViewVisualizationMode::ChildActorOnly,
						WeakEditorPtr),
					FCanExecuteAction()
				)
			);
		}
	}
}

/////////////////////////////////////////////////////
// FSubobjectEditorTreeNode

FSubobjectEditorTreeNode::FSubobjectEditorTreeNode(const FSubobjectDataHandle& SubobjectDataSource, bool InbIsSeperator /*= false*/)
	: DataHandle(SubobjectDataSource)
	, FilterFlags((uint8)EFilteredState::Unknown)
	, bIsSeperator(InbIsSeperator)
{
	check(IsValid());
}

FString FSubobjectEditorTreeNode::GetDisplayString() const
{
	FSubobjectData* Data = GetDataSource();
	return Data ? Data->GetDisplayString() : TEXT("Unknown");
}

bool FSubobjectEditorTreeNode::IsComponentNode() const
{
	FSubobjectData* Data = GetDataSource();
	return Data ? Data->IsComponent() : false;
}

bool FSubobjectEditorTreeNode::IsNativeComponent() const
{
	FSubobjectData* Data = GetDataSource();
	return Data ? Data->IsNativeComponent() : false;
}

bool FSubobjectEditorTreeNode::IsRootActorNode() const
{
	// If there is no parent node pointer then this has to be top of the tree
	return ParentNodePtr == nullptr;
}

FName FSubobjectEditorTreeNode::GetVariableName() const
{
	const FSubobjectData* const Data = GetDataSource();
	return Data->GetVariableName();
}

bool FSubobjectEditorTreeNode::CanReparent() const
{
	const FSubobjectData* const Data = GetDataSource();
	return Data->CanReparent();
}

const UActorComponent* FSubobjectEditorTreeNode::GetComponentTemplate() const
{
	const FSubobjectData* Data = GetDataSource();
	return Data ? Data->GetComponentTemplate() : nullptr;
}

const UObject* FSubobjectEditorTreeNode::GetObject(bool bEvenIfPendingKill /* = false */) const
{
	const FSubobjectData* Data = GetDataSource();
	return Data ? Data->GetObject() : nullptr;
}

FSubobjectData* FSubobjectEditorTreeNode::GetDataSource() const
{
	return DataHandle.GetSharedDataPtr().Get();
}

bool FSubobjectEditorTreeNode::IsChildSubtreeNode() const
{
	const FSubobjectData* Data = GetDataSource();
	return Data ? Data->IsChildActorSubtreeObject() : false;
}

bool FSubobjectEditorTreeNode::IsAttachedTo(FSubobjectEditorTreeNodePtrType InNodePtr) const
{
	FSubobjectEditorTreeNodePtrType TestParentPtr = ParentNodePtr;
	while (TestParentPtr.IsValid())
	{
		if (TestParentPtr == InNodePtr)
		{
			return true;
		}

		TestParentPtr = TestParentPtr->ParentNodePtr;
	}

	return false;
}

bool FSubobjectEditorTreeNode::IsDirectlyAttachedTo(FSubobjectEditorTreeNodePtrType InNodePtr) const
{
	return ParentNodePtr == InNodePtr;
}

bool FSubobjectEditorTreeNode::CanDelete() const
{
	if(const FSubobjectData* Data = GetDataSource())
	{
		return Data->CanDelete();
	}
	return false;
}

bool FSubobjectEditorTreeNode::CanRename() const
{
	if(const FSubobjectData* Data = GetDataSource())
	{
		return Data->CanRename();
	}
	return false;
}

void FSubobjectEditorTreeNode::AddChild(FSubobjectEditorTreeNodePtrType AttachToPtr)
{
	check(AttachToPtr->IsValid());
	
	// Ensure the node is not already parented elsewhere
	if (AttachToPtr->GetParent())
	{
		AttachToPtr->GetParent()->RemoveChild(AttachToPtr);
	}
	
	Children.AddUnique(AttachToPtr);
	AttachToPtr->ParentNodePtr = AsShared();
}

void FSubobjectEditorTreeNode::RemoveChild(FSubobjectEditorTreeNodePtrType InChildNodePtr)
{
	Children.Remove(InChildNodePtr);
	InChildNodePtr->ParentNodePtr.Reset();
}

FSubobjectEditorTreeNodePtrType FSubobjectEditorTreeNode::FindChild(const FSubobjectDataHandle& InHandle)
{
	for (const FSubobjectEditorTreeNodePtrType& Child : Children)
	{
		if (Child->GetDataHandle() == InHandle)
		{
			return Child;
		}
	}
	return FSubobjectEditorTreeNodePtrType();
}

bool FSubobjectEditorTreeNode::IsFlaggedForFiltration() const
{
	return FilterFlags != EFilteredState::Unknown ? (FilterFlags & EFilteredState::FilteredInMask) == 0 : false;
}

void FSubobjectEditorTreeNode::SetCachedFilterState(bool bMatchesFilter, bool bUpdateParent)
{
	bool bFlagsChanged = false;
	if ((FilterFlags & EFilteredState::Unknown) == EFilteredState::Unknown)
	{
		FilterFlags = 0x00;
		bFlagsChanged = true;
	}

	if (bMatchesFilter)
	{
		bFlagsChanged |= (FilterFlags & EFilteredState::MatchesFilter) == 0;
		FilterFlags |= EFilteredState::MatchesFilter;
	}
	else
	{
		bFlagsChanged |= (FilterFlags & EFilteredState::MatchesFilter) != 0;
		FilterFlags &= ~EFilteredState::MatchesFilter;
	}

	const bool bHadChildMatch = (FilterFlags & EFilteredState::ChildMatches) != 0;

	// refresh the cached child state (don't update the parent, we'll do that below if it's needed)
	RefreshCachedChildFilterState(/*bUpdateParent =*/false);

	bFlagsChanged |= bHadChildMatch != ((FilterFlags & EFilteredState::ChildMatches) != 0);
	if (bUpdateParent && bFlagsChanged)
	{
		ApplyFilteredStateToParent();
	}
}

void FSubobjectEditorTreeNode::ApplyFilteredStateToParent()
{
	FSubobjectEditorTreeNode* Child = this;
	while (Child->ParentNodePtr.IsValid())
	{
		FSubobjectEditorTreeNode* Parent = Child->ParentNodePtr.Get();

		if (!IsFlaggedForFiltration())
		{
			if ((Parent->FilterFlags & EFilteredState::ChildMatches) == 0)
			{
				Parent->FilterFlags |= EFilteredState::ChildMatches;
			}
			else
			{
				// all parents from here on up should have the flag
				break;
			}
		}
		// have to see if this was the only child contributing to this flag
		else if (Parent->FilterFlags & EFilteredState::ChildMatches)
		{
			Parent->FilterFlags &= ~EFilteredState::ChildMatches;
			for (const FSubobjectEditorTreeNodePtrType& Sibling : Parent->Children)
			{
				if (Sibling.Get() == Child)
				{
					continue;
				}

				if (Sibling->FilterFlags & EFilteredState::FilteredInMask)
				{
					Parent->FilterFlags |= EFilteredState::ChildMatches;
					break;
				}
			}

			if (Parent->FilterFlags & EFilteredState::ChildMatches)
			{
				// another child added the flag back
				break;
			}
		}
		Child = Parent;
	}
}

void FSubobjectEditorTreeNode::RefreshCachedChildFilterState(bool bUpdateParent)
{
	const bool bContainedMatch = !IsFlaggedForFiltration();

	FilterFlags &= ~EFilteredState::ChildMatches;
	for (FSubobjectEditorTreeNodePtrType Child : Children)
	{
		// Separator nodes should not contribute to child matches for the parent nodes
		if (Child->IsSeperator())
		{
			continue;
		}

		if (!Child->IsFlaggedForFiltration())
		{
			FilterFlags |= EFilteredState::ChildMatches;
			break;
		}
	}
	const bool bContainsMatch = !IsFlaggedForFiltration();

	const bool bStateChange = bContainedMatch != bContainsMatch;
	if (bUpdateParent && bStateChange)
	{
		ApplyFilteredStateToParent();
	}
}

bool FSubobjectEditorTreeNode::RefreshFilteredState(const UClass* InFilterType, const TArray<FString>& InFilterTerms, bool bRecursive)
{
	bool bHasAnyVisibleChildren = false;
	if (bRecursive)
	{
		for (FSubobjectEditorTreeNodePtrType Child : GetChildren())
		{
			bHasAnyVisibleChildren |= Child->RefreshFilteredState(InFilterType, InFilterTerms, bRecursive);
		}
	}

	// Don't check a root actor node - it doesn't have a valid variable name. Let it recache based on children and hide itself based on their filter states.
	if (IsRootActorNode())
	{
		SetCachedFilterState(bHasAnyVisibleChildren, /*bUpdateParent =*/!bRecursive);
		return bHasAnyVisibleChildren;
	}

	bool bIsFilteredOut = InFilterType && !MatchesFilterType(InFilterType);
	if (!bIsFilteredOut && !IsSeperator())
	{
		FString DisplayStr = GetDisplayString();
		for (const FString& FilterTerm : InFilterTerms)
		{
			if (!DisplayStr.Contains(FilterTerm))
			{
				bIsFilteredOut = true;
			}
		}
	}

	// if we're not recursing, then assume this is for a new node and we need to update the parent
	// otherwise, assume the parent was hit as part of the recursion
	const bool bUpdateParent = !bRecursive;
	SetCachedFilterState(!bIsFilteredOut, bUpdateParent);
	return !bIsFilteredOut;
}

bool FSubobjectEditorTreeNode::MatchesFilterType(const UClass* InFilterType) const
{
	if(IsComponentNode())
	{
		if (const UActorComponent* ComponentObject = GetComponentTemplate())
		{
			const UClass* ComponentClass = ComponentObject->GetClass();
			check(ComponentClass);

			if (ComponentClass->IsChildOf(InFilterType))
			{
				return true;
			}
		}
		return false;
	}
	
	// All other nodes will pass the type filter by default.
	return true;
}

void FSubobjectEditorTreeNode::SetOngoingCreateTransaction(TUniquePtr<FScopedTransaction> InTransaction)
{
	OngoingCreateTransaction = MoveTemp(InTransaction);
}

void FSubobjectEditorTreeNode::CloseOngoingCreateTransaction()
{
	OngoingCreateTransaction.Reset();
}

void FSubobjectEditorTreeNode::ForEachNodeImpl(TFunctionRef<void(FSubobjectEditorTreeNode& Data)> Ftor)
{
	for(const FSubobjectEditorTreeNodePtrType& Child : Children)
	{
		Child->ForEachNodeImpl(Ftor);
	}
	
	Ftor(*this);
}

void FSubobjectEditorTreeNode::DisposeUnsafe()
{
	if(FSubobjectData* Data = GetDataSource())
	{
		Data->DisposeUnsafe();
	}
	ParentNodePtr = FSubobjectEditorTreeNodePtrType();
}

//////////////////////////////////////////////////////////////////////////
// FSubobjectRowDragDropOp - The drag-drop operation triggered when dragging a row in the components tree

class FSubobjectRowDragDropOp : public FKismetVariableDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSubobjectRowDragDropOp, FKismetVariableDragDropAction)

	/** Available drop actions */
	enum EDropActionType
	{
		DropAction_None,
		DropAction_AttachTo,
		DropAction_DetachFrom,
		DropAction_MakeNewRoot,
		DropAction_AttachToOrMakeNewRoot
	};

	// FGraphEditorDragDropAction interface
	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	virtual FReply DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// End of FGraphEditorDragDropAction

	/** Node(s) that we started the drag from */
	TArray<FSubobjectEditorTreeNodePtrType> SourceNodes;

	/** The type of drop action that's pending while dragging */
	EDropActionType PendingDropAction;

	static TSharedRef<FSubobjectRowDragDropOp> New(FName InVariableName, UStruct* InVariableSource, FNodeCreationAnalytic AnalyticCallback);
};

TSharedRef<FSubobjectRowDragDropOp> FSubobjectRowDragDropOp::New(FName InVariableName, UStruct* InVariableSource, FNodeCreationAnalytic AnalyticCallback)
{
	TSharedPtr<FSubobjectRowDragDropOp> Operation = MakeShareable(new FSubobjectRowDragDropOp);
	Operation->VariableName = InVariableName;
	Operation->VariableSource = InVariableSource;
	Operation->AnalyticCallback = AnalyticCallback;
	Operation->Construct();
	return Operation.ToSharedRef();
}

void FSubobjectRowDragDropOp::HoverTargetChanged()
{
	bool bHoverHandled = false;

	FSlateColor IconTint = FLinearColor::White;
	const FSlateBrush* ErrorSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

	if(SourceNodes.Num() > 1)
	{
		// Display an error message if attempting to drop multiple source items onto a node
		UK2Node_Variable* VarNodeUnderCursor = Cast<UK2Node_Variable>(GetHoveredNode());
		if (VarNodeUnderCursor != nullptr)
		{
			// Icon/text to draw on tooltip
			FText Message = LOCTEXT("InvalidMultiDropTarget", "Cannot replace node with multiple nodes");
			SetSimpleFeedbackMessage(ErrorSymbol, IconTint, Message);

			bHoverHandled = true;
		}
	}

	if (!bHoverHandled)
	{
		if (SubobjectEditorUtils::ContainsChildActorSubtreeNode(SourceNodes))
		{
			// @todo - Add support for drag/drop of child actor template components to create variable get nodes in a local Blueprint event/function graph
			FText Message = LOCTEXT("ChildActorDragDropAddVariableNode_Unsupported", "This operation is not currently supported for one or more of the selected components.");
			SetSimpleFeedbackMessage(ErrorSymbol, IconTint, Message);
		}
		else if (FProperty* VariableProperty = GetVariableProperty())
		{
			const FSlateBrush* PrimarySymbol = nullptr;
			const FSlateBrush* SecondarySymbol = nullptr;
			FSlateColor PrimaryColor;
			FSlateColor SecondaryColor;
			GetDefaultStatusSymbol(/*out*/ PrimarySymbol, /*out*/ PrimaryColor, /*out*/ SecondarySymbol, /*out*/ SecondaryColor);

			// Create feedback message with the function name.
			SetSimpleFeedbackMessage(PrimarySymbol, PrimaryColor, VariableProperty->GetDisplayNameText(), SecondarySymbol, SecondaryColor);
		}
		else
		{
			FText Message = LOCTEXT("CannotFindProperty", "Cannot find corresponding variable (make sure component has been assigned to one)");
			SetSimpleFeedbackMessage(ErrorSymbol, IconTint, Message);
		}
		bHoverHandled = true;
	}

	if(!bHoverHandled)
	{
		FKismetVariableDragDropAction::HoverTargetChanged();
	}
}

FReply FSubobjectRowDragDropOp::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	// Only allow dropping on another node if there is only a single source item
	if(SourceNodes.Num() == 1)
	{
		FKismetVariableDragDropAction::DroppedOnNode(ScreenPosition, GraphPosition);
	}
	return FReply::Handled();
}

FReply FSubobjectRowDragDropOp::DroppedOnPanel(const TSharedRef<class SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	const FScopedTransaction Transaction(LOCTEXT("SSubobjectEditorAddMultipleNodes", "Add Component Nodes"));

	TArray<UK2Node_VariableGet*> OriginalVariableNodes;
	Graph.GetNodesOfClass<UK2Node_VariableGet>(OriginalVariableNodes);

	// Add source items to the graph in turn
	FVector2f GraphPositionToPlace = GraphPosition;
	for (FSubobjectEditorTreeNodePtrType& SourceNode : SourceNodes)
	{
		FSubobjectData* Data = SourceNode->GetDataSource();
		if(!Data)
		{
			continue;
		}
		
		VariableName = Data->GetVariableName();
		FKismetVariableDragDropAction::DroppedOnPanel(Panel, ScreenPosition, GraphPositionToPlace, Graph);

		GraphPositionToPlace.Y += 50.0f;
	}

	TArray<UK2Node_VariableGet*> ResultVariableNodes;
	Graph.GetNodesOfClass<UK2Node_VariableGet>(ResultVariableNodes);

	if (ResultVariableNodes.Num() - OriginalVariableNodes.Num() > 1)
	{
		TSet<const UEdGraphNode*> NodeSelection;

		// Because there is more than one new node, lets grab all the nodes at the bottom of the list and add them to a set for selection
		for (int32 NodeIdx = ResultVariableNodes.Num() - 1; NodeIdx >= OriginalVariableNodes.Num(); --NodeIdx)
		{
			NodeSelection.Add(ResultVariableNodes[NodeIdx]);
		}
		Graph.SelectNodeSet(NodeSelection);
	}
	return FReply::Handled();
}

/////////////////////////////////////////////////////
// SSubobjectEditorDragDropTree

void SSubobjectEditorDragDropTree::Construct(const FArguments& InArgs)
{
	STreeView<FSubobjectEditorTreeNodePtrType>::FArguments BaseArgs;
	BaseArgs.OnGenerateRow(InArgs._OnGenerateRow)
		.OnItemScrolledIntoView(InArgs._OnItemScrolledIntoView)
		.OnGetChildren(InArgs._OnGetChildren)
		.OnSetExpansionRecursive(InArgs._OnSetExpansionRecursive)
		.TreeItemsSource(InArgs._TreeItemsSource)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.OnMouseButtonDoubleClick(InArgs._OnMouseButtonDoubleClick)
		.OnSelectionChanged(InArgs._OnSelectionChanged)
		.SelectionMode(InArgs._SelectionMode)
		.HeaderRow(InArgs._HeaderRow)
		.ClearSelectionOnClick(InArgs._ClearSelectionOnClick)
		.ExternalScrollbar(InArgs._ExternalScrollbar)
		.OnEnteredBadState(InArgs._OnTableViewBadState)
		.HighlightParentNodesForSelection(true);

	SubobjectEditor = InArgs._SubobjectEditor;
	STreeView<FSubobjectEditorTreeNodePtrType>::Construct(BaseArgs);
}

FReply SSubobjectEditorDragDropTree::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FReply Handled = FReply::Unhandled();

	if (SubobjectEditor != nullptr)
	{
		TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
		if (Operation.IsValid() && (Operation->IsOfType<FExternalDragOperation>() || Operation->IsOfType<FAssetDragDropOp>()))
		{
			Handled = AssetUtil::CanHandleAssetDrag(DragDropEvent);

			if (!Handled.IsEventHandled())
			{
				if (Operation->IsOfType<FAssetDragDropOp>())
				{
					const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

					for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
					{
						if (UClass* AssetClass = AssetData.GetClass())
						{
							if (AssetClass->IsChildOf(UClass::StaticClass()))
							{
								Handled = FReply::Handled();
								break;
							}
						}
					}
				}
			}
		}
	}

	return Handled;
}

FReply SSubobjectEditorDragDropTree::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (SubobjectEditor != nullptr)
	{
		return SubobjectEditor->TryHandleAssetDragDropOperation(DragDropEvent);
	}
	else
	{
		return FReply::Unhandled();
	}
}

/////////////////////////////////////////////////////
// SSubobject_RowWidget

void SSubobject_RowWidget::Construct(const FArguments& InArgs, TWeakPtr<SSubobjectEditor> InEditor, FSubobjectEditorTreeNodePtrType InNodePtr, TSharedPtr<STableViewBase> InOwnerTableView)
{
	check(InNodePtr.IsValid());
	SubobjectEditor = InEditor;
	SubobjectPtr = InNodePtr;

	bool bIsSeparator = InNodePtr->IsSeperator();

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.Style(bIsSeparator ?
			&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.NoHoverTableRow") :
			&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow")) //@todo create editor style for the SCS tree
		.Padding(FMargin(0.f, 4.f, 0.f, 4.f))
		.ShowSelection(!bIsSeparator)
		.OnDragDetected(this, &SSubobject_RowWidget::HandleOnDragDetected)
		.OnDragEnter(this, &SSubobject_RowWidget::HandleOnDragEnter)
		.OnDragLeave(this, &SSubobject_RowWidget::HandleOnDragLeave)
		.OnCanAcceptDrop(this, &SSubobject_RowWidget::HandleOnCanAcceptDrop)
		.OnAcceptDrop(this, &SSubobject_RowWidget::HandleOnAcceptDrop);

	SMultiColumnTableRow<FSubobjectEditorTreeNodePtrType>::Construct(Args, InOwnerTableView.ToSharedRef());
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SSubobject_RowWidget::GenerateWidgetForColumn(const FName& ColumnName)
{
	FSubobjectEditorTreeNodePtrType Node = GetSubobjectPtr();
	check(Node);
	
	if (Node->IsSeperator())
	{
		return SNew(SBox)
			.Padding(1.f)
			[
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::Get().GetBrush("Menu.Separator"))
				.Thickness(1.f)
		 	];
	}

	FSubobjectData* Data = Node->GetDataSource();

	if (ColumnName == SubobjectTree_ColumnName_ComponentClass)
	{
		InlineWidget =
			SNew(SInlineEditableTextBlock)
			.Text(this, &SSubobject_RowWidget::GetNameLabel)
			.OnVerifyTextChanged(this, &SSubobject_RowWidget::OnNameTextVerifyChanged)
			.OnTextCommitted(this, &SSubobject_RowWidget::OnNameTextCommit)
			.IsSelected(this, &SSubobject_RowWidget::IsSelectedExclusively)
			.IsReadOnly(this, &SSubobject_RowWidget::IsReadOnly);

		Node->SetRenameRequestedDelegate(FSubobjectEditorTreeNode::FOnRenameRequested::CreateSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode));
		InlineWidget->AddMetadata<FTagMetaData>(MakeShared<FTagMetaData>(SubobjectEditor_ClearFocusOnDuplicate));
		
		const bool IsRootActorNode = Data->IsRootActor();
		
		TSharedRef<SHorizontalBox> Contents = 
		SNew(SHorizontalBox)
			.ToolTip(CreateToolTipWidget())
			+ SHorizontalBox::Slot()
			.Padding(FMargin(IsRootActorNode ? 0.f : 4.f, 0.f, 0.f, 0.f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SExpanderArrow, SharedThis(this))
				.Visibility(IsRootActorNode ? EVisibility::Collapsed : EVisibility::Visible)
			]
		+ SHorizontalBox::Slot()
			.Padding(FMargin(IsRootActorNode ? 4.f : 0.f, 0.f, 0.f, 0.f))
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(GetIconBrush())
				.ColorAndOpacity(this, &SSubobject_RowWidget::GetColorTintForIcon)
			]
		+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(6.f, 0.f, 0.f, 0.f)
			[
				InlineWidget.ToSharedRef()
			]
		+ SHorizontalBox::Slot()
	        .HAlign(HAlign_Left)
	        .VAlign(VAlign_Center)
			.FillWidth(1.0f)
	        .Padding(4.f, 0.f, 0.f, 0.f)
	        [
	            SNew(STextBlock)
	            .Text(this, &SSubobject_RowWidget::GetObjectContextText)
	            .ColorAndOpacity(FSlateColor::UseForeground())
	        ]
		+ SHorizontalBox::Slot()
	        .HAlign(HAlign_Right)
	        .VAlign(VAlign_Center)
			.AutoWidth()
	        .Padding(4.f, 0.f, 4.f, 0.f)
	        [
	            GetInheritedLinkWidget()
	        ];
				
		return Contents;
	}
	else if (ColumnName == SubobjectTree_ColumnName_Asset)
	{
		return
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(this, &SSubobject_RowWidget::GetAssetVisibility)
				.Text(this, &SSubobject_RowWidget::GetAssetName)
				.ToolTipText(this, &SSubobject_RowWidget::GetAssetPath)
			];
	}
	else if (ColumnName == SubobjectTree_ColumnName_Mobility)
	{
		if (Data && Data->GetComponentTemplate())
		{
			TSharedPtr<SToolTip> MobilityTooltip = SNew(SToolTip)
				.Text(this, &SSubobject_RowWidget::GetMobilityToolTipText);

			return SNew(SHorizontalBox)
				.ToolTip(MobilityTooltip)
				.Visibility(EVisibility::Visible) // so we still get tooltip text for an empty SHorizontalBox
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					SNew(SImage)
					.Image(this, &SSubobject_RowWidget::GetMobilityIconImage)
					.ToolTip(MobilityTooltip)
				];
		}
		else
		{
			return SNew(SSpacer);
		}
	}

	return SNew(STextBlock)
		.Text(LOCTEXT("UnknownColumn", "Unknown Column"));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SSubobject_RowWidget::GetNameLabel() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if(Data)
	{
		// NOTE: Whatever this returns also becomes the variable name
		return FText::FromString(NodePtr->GetDisplayString());
	}
	return LOCTEXT("UnknownDataSource", "Unknown Name Label");
}

FText SSubobject_RowWidget::GetAssetPath() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	FSubobjectData* Data = NodePtr->GetDataSource();

	return Data ? Data->GetAssetPath() : FText::GetEmpty();
}

FText SSubobject_RowWidget::GetAssetName() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	if(FSubobjectData* Data = NodePtr->GetDataSource())
	{
		return Data->GetAssetName();

	}
	return FText::GetEmpty();
}

EVisibility SSubobject_RowWidget::GetAssetVisibility() const
{	
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;

	if (Data && Data->IsAssetVisible())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

const FSlateBrush* SSubobject_RowWidget::GetIconBrush() const
{
	const FSlateBrush* ComponentIcon = FAppStyle::GetBrush("SCS.NativeComponent");

	if (FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr())
	{
		if (const FSubobjectData* Data = NodePtr->GetDataSource())
		{
			TSharedPtr<SSubobjectEditor> SubobjectEditorPinned = SubobjectEditor.Pin();
			const ISCSEditorUICustomization* UICustomization = SubobjectEditorPinned ? SubobjectEditorPinned->GetUICustomization() : nullptr;
			if (UICustomization)
			{
				const FSlateBrush* SubobjectIcon = UICustomization->GetIconBrush(*Data);
				if (SubobjectIcon)
				{
					return SubobjectIcon;
				}
			}
			
			if (const AActor* Actor = Data->GetObject<AActor>())
			{
				return FClassIconFinder::FindIconForActor(Actor);
			}
			else if (const UActorComponent* ComponentTemplate = Data->GetComponentTemplate())
			{
				ComponentIcon = FSlateIconFinder::FindIconBrushForClass(ComponentTemplate->GetClass(), TEXT("SCS.Component"));
			}
		}
	}

	return ComponentIcon;
}

FSlateColor SSubobject_RowWidget::GetColorTintForIcon() const
{
	return SubobjectEditor.Pin()->GetColorTintForIcon(GetSubobjectPtr());
}

ESelectionMode::Type SSubobject_RowWidget::GetSelectionMode() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	if (NodePtr && NodePtr->IsSeperator())
	{
		return ESelectionMode::None;
	}
	
	return SMultiColumnTableRow<FSubobjectEditorTreeNodePtrType>::GetSelectionMode();
}

void SSubobject_RowWidget::HandleOnDragEnter(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return;
	}
	TSharedPtr<FSubobjectRowDragDropOp> DragRowOp = DragDropEvent.GetOperationAs<FSubobjectRowDragDropOp>();

	if (DragRowOp.IsValid())
	{
		check(SubobjectEditor.IsValid());
		
		FText Message;
		FSlateColor IconColor = FLinearColor::White;

		for (const FSubobjectEditorTreeNodePtrType& SelectedNodePtr : DragRowOp->SourceNodes)
		{
			const FSubobjectData* Data = SelectedNodePtr->GetDataSource();
			if (!Data->CanReparent())
			{
				// We set the tooltip text here because it won't change across entry/leave events
				if (DragRowOp->SourceNodes.Num() == 1)
				{
					if (SelectedNodePtr->IsChildSubtreeNode())
					{
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparent_ChildActorSubTreeNodes", "The selected component is part of a child actor template and cannot be reordered here.");
					}
					else if (!Data->IsSceneComponent())
					{
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparent_NotSceneComponent", "The selected component is not a scene component and cannot be attached to other components.");
					}
					else if (Data->IsInheritedComponent())
					{
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparent_Inherited", "The selected component is inherited and cannot be reordered here.");
					}
					else
					{
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparent", "The selected component cannot be moved.");
					}
				}
				else
				{
					Message = LOCTEXT("DropActionToolTip_Error_CannotReparentMultiple", "One or more of the selected components cannot be attached.");
				}
				break;
			}
		}

		if (Message.IsEmpty())
		{
			FSubobjectEditorTreeNodePtrType SceneRootNodePtr = SubobjectEditor.Pin()->GetSceneRootNode();

			FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
			const FSubobjectData* NodeData = NodePtr->GetDataSource();

			if (!NodePtr->IsComponentNode())
			{
				// Don't show a feedback message if over a node that makes no sense, such as a separator or the instance node
				Message = LOCTEXT("DropActionToolTip_FriendlyError_DragToAComponent", "Drag to another component in order to attach to that component or become the root component.\nDrag to a Blueprint graph in order to drop a reference.");
			}
			else if (NodePtr->IsChildSubtreeNode())
			{
				// Can't drag onto components within a child actor node's subtree
				Message = LOCTEXT("DropActionToolTip_Error_ChildActorSubTree", "Cannot attach to this component as it is part of a child actor template.");
			}
			
			// Validate each selected node being dragged against the node that belongs to this row. Exit the loop if we have a valid tooltip OR a valid pending drop action once all nodes in the selection have been validated.
			for (auto SourceNodeIter = DragRowOp->SourceNodes.CreateConstIterator(); SourceNodeIter && (Message.IsEmpty() || DragRowOp->PendingDropAction != FSubobjectRowDragDropOp::DropAction_None); ++SourceNodeIter)
			{
				FSubobjectEditorTreeNodePtrType DraggedNodePtr = *SourceNodeIter;
				const FSubobjectData* const DraggedNodeData = DraggedNodePtr->GetDataSource();
				check(DraggedNodePtr.IsValid());

				// Reset the pending drop action each time through the loop
				DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_None;

				// Get the component template objects associated with each node
				const USceneComponent* HoveredTemplate = Cast<USceneComponent>(NodePtr->GetComponentTemplate());
				const USceneComponent* DraggedTemplate = Cast<USceneComponent>(DraggedNodeData->GetComponentTemplate());

				if (DraggedNodePtr == NodePtr)
				{
					// Attempted to drag and drop onto self
					if (DragRowOp->SourceNodes.Num() > 1)
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_Error_CannotAttachToSelfWithMultipleSelection", "Cannot attach the selected components here because it would result in {0} being attached to itself. Remove it from the selection and try again."), DraggedNodeData->GetDragDropDisplayText());
					}
					else
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_Error_CannotAttachToSelf", "Cannot attach {0} to itself."), DraggedNodeData->GetDragDropDisplayText());
					}
				}
				else if (NodePtr->IsAttachedTo(DraggedNodePtr))
				{
					// Attempted to drop a parent onto a child
					if (DragRowOp->SourceNodes.Num() > 1)
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_Error_CannotAttachToChildWithMultipleSelection", "Cannot attach the selected components here because it would result in {0} being attached to one of its children. Remove it from the selection and try again."), DraggedNodeData->GetDragDropDisplayText());
					}
					else
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_Error_CannotAttachToChild", "Cannot attach {0} to one of its children."), DraggedNodeData->GetDragDropDisplayText());
					}
				}
				else if (HoveredTemplate == nullptr || DraggedTemplate == nullptr)
				{
					if (HoveredTemplate == nullptr)
					{
						// Can't attach to non-USceneComponent types
						Message = LOCTEXT("DropActionToolTip_Error_NotAttachable_NotSceneComponent", "Cannot attach to this component as it is not a scene component.");
					}
					else
					{
						// Can't attach non-USceneComponent types
						Message = LOCTEXT("DropActionToolTip_Error_NotAttachable", "Cannot attach to this component.");
					}
				}
				else if (NodePtr == SceneRootNodePtr)
				{
					bool bCanMakeNewRoot = false;
					bool bCanAttachToRoot = !DraggedNodePtr->IsDirectlyAttachedTo(NodePtr)
						&& HoveredTemplate->CanAttachAsChild(DraggedTemplate, NAME_None)
						&& DraggedTemplate->Mobility >= HoveredTemplate->Mobility
						&& (!HoveredTemplate->IsEditorOnly() || DraggedTemplate->IsEditorOnly());

					if (!NodePtr->CanReparent() && (!NodeData->IsDefaultSceneRoot() || NodeData->IsInheritedComponent() || NodeData->IsInstancedInheritedComponent()))
					{
						// Cannot make the dropped node the new root if we cannot reparent the current root
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparentRootNode", "The root component in this Blueprint is inherited and cannot be replaced.");
					}
					else if (DraggedTemplate->IsEditorOnly() && !HoveredTemplate->IsEditorOnly())
					{
						// can't have a new root that's editor-only (when children would be around in-game)
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparentEditorOnly", "Cannot re-parent game components under editor-only ones.");
					}
					else if (DraggedTemplate->Mobility > HoveredTemplate->Mobility)
					{
						// can't have a new root that's movable if the existing root is static or stationary
						Message = LOCTEXT("DropActionToolTip_Error_CannotReparentNonMovable", "Cannot replace a non-movable scene root with a movable component.");
					}
					else if (DragRowOp->SourceNodes.Num() > 1)
					{
						Message = LOCTEXT("DropActionToolTip_Error_CannotAssignMultipleRootNodes", "Cannot replace the scene root with multiple components. Please select only a single component and try again.");
					}
					else
					{
						bCanMakeNewRoot = true;
					}

					if (bCanMakeNewRoot && bCanAttachToRoot)
					{
						// User can choose to either attach to the current root or make the dropped node the new root
						Message = LOCTEXT("DropActionToolTip_AttachToOrMakeNewRoot", "Drop here to see available actions.");
						DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_AttachToOrMakeNewRoot;
					}
					else if (SubobjectEditor.Pin()->CanMakeNewRootOnDrag(DraggedNodeData->GetBlueprint()))
					{
						if (bCanMakeNewRoot)
						{
							if (NodeData->IsDefaultSceneRoot())
							{
								// Only available action is to copy the dragged node to the other Blueprint and make it the new root
								// Default root will be deleted
								Message = FText::Format(LOCTEXT("DropActionToolTip_DropMakeNewRootNodeFromCopyAndDelete", "Drop here to copy {0} to a new variable and make it the new root. The default root will be deleted."), DraggedNodeData->GetDragDropDisplayText());
							}
							else
							{
								// Only available action is to copy the dragged node to the other Blueprint and make it the new root
								Message = FText::Format(LOCTEXT("DropActionToolTip_DropMakeNewRootNodeFromCopy", "Drop here to copy {0} to a new variable and make it the new root."), DraggedNodeData->GetDragDropDisplayText());
							}
							DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_MakeNewRoot;
						}
						else if (bCanAttachToRoot)
						{
							// Only available action is to copy the dragged node(s) to the other Blueprint and attach it to the root
							if (DragRowOp->SourceNodes.Num() > 1)
							{
								Message = FText::Format(LOCTEXT("DropActionToolTip_AttachComponentsToThisNodeFromCopyWithMultipleSelection", "Drop here to copy the selected components to new variables and attach them to {0}."), NodeData->GetDragDropDisplayText());
							}
							else
							{
								Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNodeFromCopy", "Drop here to copy {0} to a new variable and attach it to {1}."), DraggedNodeData->GetDragDropDisplayText(), NodeData->GetDragDropDisplayText());
							}

							DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_AttachTo;
						}
					}
					else if (bCanMakeNewRoot)
					{
						if (NodeData->IsDefaultSceneRoot())
						{
							// Only available action is to make the dragged node the new root
							// Default root will be deleted
							Message = FText::Format(LOCTEXT("DropActionToolTip_DropMakeNewRootNodeAndDelete", "Drop here to make {0} the new root. The default root will be deleted."), DraggedNodeData->GetDragDropDisplayText());
						}
						else
						{
							// Only available action is to make the dragged node the new root
							Message = FText::Format(LOCTEXT("DropActionToolTip_DropMakeNewRootNode", "Drop here to make {0} the new root."), DraggedNodeData->GetDragDropDisplayText());
						}
						DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_MakeNewRoot;
					}
					else if (bCanAttachToRoot)
					{
						// Only available action is to attach the dragged node(s) to the root
						if (DragRowOp->SourceNodes.Num() > 1)
						{
							Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNodeWithMultipleSelection", "Drop here to attach the selected components to {0}."), NodeData->GetDragDropDisplayText());
						}
						else
						{
							Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNode", "Drop here to attach {0} to {1}."), DraggedNodeData->GetDragDropDisplayText(), NodeData->GetDragDropDisplayText());
						}

						DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_AttachTo;
					}
				}
				else if (DraggedNodePtr->IsDirectlyAttachedTo(NodePtr)) // if dropped onto parent
				{
					// Detach the dropped node(s) from the current node and reattach to the root node
					if (DragRowOp->SourceNodes.Num() > 1)
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_DetachFromThisNodeWithMultipleSelection", "Drop here to detach the selected components from {0}."), NodeData->GetDragDropDisplayText());
					}
					else
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_DetachFromThisNode", "Drop here to detach {0} from {1}."), DraggedNodeData->GetDragDropDisplayText(), NodeData->GetDragDropDisplayText());
					}

					DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_DetachFrom;
				}
				else if (!DraggedTemplate->IsEditorOnly() && HoveredTemplate->IsEditorOnly())
				{
					// can't have a game component child nested under an editor-only one
					Message = LOCTEXT("DropActionToolTip_Error_CannotAttachToEditorOnly", "Cannot attach game components to editor-only ones.");
				}
				else if ((DraggedTemplate->Mobility == EComponentMobility::Static) && ((HoveredTemplate->Mobility == EComponentMobility::Movable) || (HoveredTemplate->Mobility == EComponentMobility::Stationary)))
				{
					// Can't attach Static components to mobile ones
					Message = LOCTEXT("DropActionToolTip_Error_CannotAttachStatic", "Cannot attach Static components to movable ones.");
				}
				else if ((DraggedTemplate->Mobility == EComponentMobility::Stationary) && (HoveredTemplate->Mobility == EComponentMobility::Movable))
				{
					// Can't attach Static components to mobile ones
					Message = LOCTEXT("DropActionToolTip_Error_CannotAttachStationary", "Cannot attach Stationary components to movable ones.");
				}
				else if ((NodeData->IsInstancedComponent() && HoveredTemplate->CreationMethod == EComponentCreationMethod::Native && !HoveredTemplate->HasAnyFlags(RF_DefaultSubObject)))
				{
					// Can't attach to post-construction C++-added components as they exist outside of the CDO and are not known at SCS execution time
					Message = LOCTEXT("DropActionToolTip_Error_CannotAttachCPPAdded", "Cannot attach to components added in post-construction C++ code.");
				}
				else if (NodeData->IsInstancedComponent() && HoveredTemplate->CreationMethod == EComponentCreationMethod::UserConstructionScript)
				{
					// Can't attach to UCS-added components as they exist outside of the CDO and are not known at SCS execution time
					Message = LOCTEXT("DropActionToolTip_Error_CannotAttachUCSAdded", "Cannot attach to components added in the Construction Script.");
				}
				else if (HoveredTemplate->CanAttachAsChild(DraggedTemplate, NAME_None))
				{
					// Attach the dragged node(s) to this node
					if (DraggedNodeData->GetBlueprint() != NodeData->GetBlueprint())
					{
						if (DragRowOp->SourceNodes.Num() > 1)
						{
							Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNodeFromCopyWithMultipleSelection", "Drop here to copy the selected nodes to new variables and attach them to {0}."), NodeData->GetDragDropDisplayText());
						}
						else
						{
							Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNodeFromCopy", "Drop here to copy {0} to a new variable and attach it to {1}."), DraggedNodeData->GetDragDropDisplayText(), NodeData->GetDragDropDisplayText());
						}
					}
					else if (DragRowOp->SourceNodes.Num() > 1)
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNodeWithMultipleSelection", "Drop here to attach the selected components to {0}."), NodeData->GetDragDropDisplayText());
					}
					else
					{
						Message = FText::Format(LOCTEXT("DropActionToolTip_AttachToThisNode", "Drop here to attach {0} to {1}."), DraggedNodeData->GetDragDropDisplayText(), NodeData->GetDragDropDisplayText());
					}

					DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_AttachTo;
				}
				else
				{
					// The dropped node cannot be attached to the current node
					Message = FText::Format(LOCTEXT("DropActionToolTip_Error_TooManyAttachments", "Unable to attach {0} to {1}."), DraggedNodeData->GetDragDropDisplayText(), NodeData->GetDragDropDisplayText());
				}
			}
		}
		
		const FSlateBrush* StatusSymbol = DragRowOp->PendingDropAction != FSubobjectRowDragDropOp::DropAction_None
			? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"))
			: FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	
		if (Message.IsEmpty())
		{
			DragRowOp->SetFeedbackMessage(nullptr);
		}
		else
		{
			DragRowOp->SetSimpleFeedbackMessage(StatusSymbol, FLinearColor::White, Message);
		}
	}
	else if (Operation->IsOfType<FExternalDragOperation>() || Operation->IsOfType<FAssetDragDropOp>())
	{
		// defer to the tree widget's handler for this type of operation
		TSharedPtr<SSubobjectEditor> PinnedEditor = SubobjectEditor.Pin();
		if ( PinnedEditor.IsValid() && PinnedEditor->GetDragDropTree().IsValid() )
		{
			// The widget geometry is irrelevant to the tree widget's OnDragEnter
			PinnedEditor->GetDragDropTree()->OnDragEnter( FGeometry(), DragDropEvent );
		}
	}
}

void SSubobject_RowWidget::HandleOnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FSubobjectRowDragDropOp> DragRowOp = DragDropEvent.GetOperationAs<FSubobjectRowDragDropOp>();
	if (DragRowOp.IsValid())
	{
		bool bCanReparentAllNodes = true;
		for(auto SourceNodeIter = DragRowOp->SourceNodes.CreateConstIterator(); SourceNodeIter && bCanReparentAllNodes; ++SourceNodeIter)
		{
			FSubobjectEditorTreeNodePtrType DraggedNodePtr = *SourceNodeIter;
			check(DraggedNodePtr.IsValid());
			const FSubobjectData* Data = DraggedNodePtr->GetDataSource();

			bCanReparentAllNodes = Data->CanReparent();
		}

		// Only clear the tooltip text if all dragged nodes support it
		if(bCanReparentAllNodes)
		{
			TSharedPtr<SWidget> NoWidget;
			DragRowOp->SetFeedbackMessage(NoWidget);
			DragRowOp->PendingDropAction = FSubobjectRowDragDropOp::DropAction_None;
		}
	}
}

FReply SSubobject_RowWidget::HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<SSubobjectEditor> SubobjectEditorPtr = SubobjectEditor.Pin();
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton)
        && SubobjectEditorPtr.IsValid()
        && SubobjectEditorPtr->IsEditingAllowed()) //can only drag when editing
    {
		TArray<FSubobjectEditorTreeNodePtrType> SelectedNodePtrs = SubobjectEditorPtr->GetSelectedNodes();
		if (SelectedNodePtrs.Num() == 0)
		{
			SelectedNodePtrs.Add(GetSubobjectPtr());
		}

		FSubobjectEditorTreeNodePtrType FirstNode = SelectedNodePtrs[0];
		const FSubobjectData* Data = FirstNode->GetDataSource();

		if (Data && FirstNode->IsComponentNode())
		{
			// Do not use the Blueprint from FirstNode, it may still be referencing the parent.
			UBlueprint* Blueprint = SubobjectEditorPtr->GetBlueprint();
			const FName VariableName = Data->GetVariableName();
			UStruct* VariableScope = (Blueprint != nullptr) ? *Blueprint->SkeletonGeneratedClass : nullptr;

			TSharedRef<FSubobjectRowDragDropOp> Operation = FSubobjectRowDragDropOp::New(VariableName, VariableScope, FNodeCreationAnalytic());
			Operation->SetCtrlDrag(true); // Always put a getter
			Operation->PendingDropAction = FSubobjectRowDragDropOp::DropAction_None;
			Operation->SourceNodes = SelectedNodePtrs;

			return FReply::Handled().BeginDragDrop(Operation);
		}
    }
	
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SSubobject_RowWidget::HandleOnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FSubobjectEditorTreeNodePtrType TargetItem)
{
	TOptional<EItemDropZone> ReturnDropZone;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		if (Operation->IsOfType<FSubobjectRowDragDropOp>() && ( Cast<USceneComponent>(GetSubobjectPtr()->GetComponentTemplate()) != nullptr ))
		{
			TSharedPtr<FSubobjectRowDragDropOp> DragRowOp = StaticCastSharedPtr<FSubobjectRowDragDropOp>(Operation);
			check(DragRowOp.IsValid());

			if (DragRowOp->PendingDropAction != FSubobjectRowDragDropOp::DropAction_None)
			{
				ReturnDropZone = EItemDropZone::OntoItem;
			}
		}
		else if (Operation->IsOfType<FExternalDragOperation>() || Operation->IsOfType<FAssetDragDropOp>())
		{
			ReturnDropZone = EItemDropZone::OntoItem;
		}
	}

	return ReturnDropZone;
}

FReply SSubobject_RowWidget::HandleOnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FSubobjectEditorTreeNodePtrType TargetItem)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<SSubobjectEditor> SubobjectEditorPtr = SubobjectEditor.Pin();
	if(!SubobjectEditorPtr.IsValid())
	{
		return FReply::Handled();
	}
	
	if (Operation->IsOfType<FSubobjectRowDragDropOp>() && (Cast<USceneComponent>(GetSubobjectPtr()->GetComponentTemplate()) != nullptr))
	{
		TSharedPtr<FSubobjectRowDragDropOp> DragRowOp = StaticCastSharedPtr<FSubobjectRowDragDropOp>(Operation);	
		check(DragRowOp.IsValid());

		switch(DragRowOp->PendingDropAction)
		{
		case FSubobjectRowDragDropOp::DropAction_AttachTo:
			SubobjectEditorPtr->OnAttachToDropAction(GetSubobjectPtr(), DragRowOp->SourceNodes);
			break;
			
		case FSubobjectRowDragDropOp::DropAction_DetachFrom:
			SubobjectEditorPtr->OnDetachFromDropAction(DragRowOp->SourceNodes);
			break;

		case FSubobjectRowDragDropOp::DropAction_MakeNewRoot:
			check(DragRowOp->SourceNodes.Num() == 1);
			SubobjectEditorPtr->OnMakeNewRootDropAction(DragRowOp->SourceNodes[0]);
			break;

		case FSubobjectRowDragDropOp::DropAction_AttachToOrMakeNewRoot:
			{
				check(DragRowOp->SourceNodes.Num() == 1);
				FWidgetPath WidgetPath = DragDropEvent.GetEventPath() != nullptr ? *DragDropEvent.GetEventPath() : FWidgetPath();
				FSlateApplication::Get().PushMenu(
                    SharedThis(this),
                    WidgetPath,
                    SubobjectEditorPtr->BuildSceneRootDropActionMenu(GetSubobjectPtr(), DragRowOp->SourceNodes[0]).ToSharedRef(),
                    FSlateApplication::Get().GetCursorPos(),
                    FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
                );
			}
			break;

		case FSubobjectRowDragDropOp::DropAction_None:
		default:
            break;
		}
	}
	else if (Operation->IsOfType<FExternalDragOperation>() || Operation->IsOfType<FAssetDragDropOp>())
	{
		// defer to the tree widget's handler for this type of operation
		TSharedPtr<SSubobjectEditor> PinnedEditor = SubobjectEditor.Pin();
		TSharedPtr<SSubobjectEditorDragDropTree> TreeWidget = PinnedEditor.IsValid() ? PinnedEditor->GetDragDropTree() : nullptr;
		if (TreeWidget.IsValid())
		{
			return PinnedEditor->TryHandleAssetDragDropOperation(DragDropEvent, TargetItem);
		}
	}
	
	return FReply::Handled();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSubobject_RowWidget::AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, TSharedRef<SWidget> ValueIcon, const TAttribute<FText>& Value, bool bImportant)
{
	InfoBox->AddSlot()
	       .AutoHeight()
	       .Padding(0, 1)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		  .AutoWidth()
		  .Padding(0, 0, 4, 0)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(),
			           bImportant ? "SCSEditor.ComponentTooltip.ImportantLabel" : "SCSEditor.ComponentTooltip.Label")
		.Text(FText::Format(LOCTEXT("AssetViewTooltipFormat", "{0}:"), Key))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ValueIcon
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(),
			           bImportant ? "SCSEditor.ComponentTooltip.ImportantValue" : "SCSEditor.ComponentTooltip.Value")
		.Text(Value)
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SToolTip> SSubobject_RowWidget::CreateToolTipWidget() const
{
	FSubobjectEditorTreeNodePtrType TreeNode = GetSubobjectPtr();
	check(TreeNode);
	const FSubobjectData* TreeNodeData = TreeNode->GetDataSource();
	check(TreeNodeData);
	
	if (TreeNode->IsComponentNode())
	{
		return CreateComponentTooltipWidget(TreeNode);
	}
	else
	{
		return CreateActorTooltipWidget(TreeNode);
	}
}

TSharedRef<SToolTip> SSubobject_RowWidget::CreateComponentTooltipWidget(const FSubobjectEditorTreeNodePtrType& InNode) const
{
	// Create a box to hold every line of info in the body of the tooltip
	TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

	if (FSubobjectEditorTreeNodePtrType TreeNode = GetSubobjectPtr())
	{
		const FSubobjectData* TreeNodeData = TreeNode->GetDataSource();
		check(TreeNodeData);
		if (TreeNode->IsComponentNode())
		{
			if (const UActorComponent* Template = TreeNode->GetComponentTemplate())
			{		
				UClass* TemplateClass = Template->GetClass();
				FText ClassTooltip = TemplateClass->GetToolTipText(/*bShortTooltip=*/ true);

				InfoBox->AddSlot()
				       .AutoHeight()
				       .HAlign(HAlign_Center)
				       .Padding(FMargin(0, 2, 0, 4))
				[
					SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "SCSEditor.ComponentTooltip.ClassDescription")
						.Text(ClassTooltip)
						.WrapTextAt(400.0f)
				];
			}

			// Add introduction point
			AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipAddType", "Source"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetComponentAddSourceToolTipText)), false);
			if (TreeNodeData->IsInheritedComponent())
			{
				AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipIntroducedIn", "Introduced in"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetIntroducedInToolTipText)), false);
			}

			// Add Underlying Component Name for Native Components
			if (TreeNodeData->IsNativeComponent())
			{
				AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipNativeComponentName", "Native Component Name"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetNativeComponentNameToolTipText)), false);
			}

			// Add mobility
			TSharedRef<SImage> MobilityIcon = SNew(SImage).Image(this, &SSubobject_RowWidget::GetMobilityIconImage);
			AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipMobility", "Mobility"), MobilityIcon, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetMobilityToolTipText)), false);

			// Add asset if applicable to this node
			if (GetAssetVisibility() == EVisibility::Visible)
			{
				InfoBox->AddSlot()[SNew(SSpacer).Size(FVector2D(1.0f, 8.0f))];
				AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipAsset", "Asset"), SNullWidget::NullWidget,
				                    TAttribute<FText>(this, &SSubobject_RowWidget::GetAssetName), false);
			}

			// If the component is marked as editor only, then display that info here
			AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipEditorOnly", "Editor Only"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetComponentEditorOnlyTooltipText)), false);
		}
	}

	TSharedRef<SBorder> TooltipContent = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(0)
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		  .AutoHeight()
		  .Padding(0, 0, 0, 4)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				  .AutoWidth()
				  .VAlign(VAlign_Center)
				  .Padding(2)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "SCSEditor.ComponentTooltip.Title")
					.Text(this, &SSubobject_RowWidget::GetToolTipText)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(2)
			[
				InfoBox
			]
		]
	];

	return IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SSubobject_RowWidget::GetToolTipText),
	                                            TooltipContent, InfoBox, GetDocumentationLink(),
	                                            GetDocumentationExcerptName());
}

TSharedRef<SToolTip> SSubobject_RowWidget::CreateActorTooltipWidget(const FSubobjectEditorTreeNodePtrType& InNode) const
{
	// Create a box to hold every line of info in the body of the tooltip
	TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

	// Add class
	AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipClass", "Class"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetActorClassNameText)), false);

	// Add super class
	AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipSuperClass", "Parent Class"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetActorSuperClassNameText)), false);

	// Add mobility
	AddToToolTipInfoBox(InfoBox, LOCTEXT("TooltipMobility", "Mobility"), SNullWidget::NullWidget, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SSubobject_RowWidget::GetActorMobilityText)), false);

	TSharedRef<SBorder> TooltipContent = SNew(SBorder)
        .BorderImage(FAppStyle::GetBrush("NoBorder"))
        .Padding(0)
        [
            SNew(SVerticalBox)

            + SVerticalBox::Slot()
            .AutoHeight()
            .Padding(0, 0, 0, 4)
            [
                SNew(SVerticalBox)

                + SVerticalBox::Slot()
                .AutoHeight()
                [
                    SNew(SHorizontalBox)

                    + SHorizontalBox::Slot()
                    .AutoWidth()
                    .VAlign(VAlign_Center)
                    .Padding(4)
                    [
                        SNew(STextBlock)
                        .TextStyle(FAppStyle::Get(), "SCSEditor.ComponentTooltip.Title")
                        .Text(this, &SSubobject_RowWidget::GetActorDisplayText)
                    ]
                ]
            ]

            + SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(SBorder)
                .BorderImage(FAppStyle::GetBrush("NoBorder"))
                .Padding(4)
                [
                    InfoBox
                ]
            ]
        ];

	return IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SSubobject_RowWidget::GetActorDisplayText), TooltipContent, InfoBox, TEXT(""), TEXT(""));
}

FText SSubobject_RowWidget::GetToolTipText() const
{
	const FSubobjectData* Data = GetSubobjectPtr()->GetDataSource();

	if(!Data)
	{
		return FText::GetEmpty();
	}
	
	if (Data->IsDefaultSceneRoot())
	{
		if (Data->IsInheritedComponent())
		{
			return LOCTEXT("InheritedDefaultSceneRootToolTip",
			               "This is the default scene root component. It cannot be renamed or deleted.\nIt has been inherited from the parent class, so its properties cannot be edited here.\nNew scene components will automatically be attached to it.");
		}
		else
		{
			if (Data->CanDelete())
			{
				return LOCTEXT("DefaultSceneRootDeletableToolTip",
					"This is the default scene root component.\nIt can be replaced by drag/dropping another scene component over it.");
			}
			else
			{
				return LOCTEXT("DefaultSceneRootToolTip",
					"This is the default scene root component. It cannot be renamed or deleted.\nIt can be replaced by drag/dropping another scene component over it.");
			}
		}
	}
	else
	{
		return Data->GetToolTipText();
	}
}

FText SSubobject_RowWidget::GetActorClassNameText() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if(Data)
	{
		if (const AActor* DefaultActor = Data->GetObject<AActor>())
		{
			return FText::FromString(DefaultActor->GetClass()->GetName());
		}
	}
	
	return FText::GetEmpty();
}

FText SSubobject_RowWidget::GetActorSuperClassNameText() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if (Data)
	{
		if (const AActor* DefaultActor = Data->GetObject<AActor>())
		{
			return FText::FromString(DefaultActor->GetClass()->GetSuperClass()->GetName());
		}
	}

	return FText::GetEmpty();
}

FText SSubobject_RowWidget::GetActorMobilityText() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if (Data)
	{
		if (const AActor* DefaultActor = Data->GetObject<AActor>())
		{
			USceneComponent* RootComponent = DefaultActor->GetRootComponent();

			if (RootComponent != nullptr)
			{
				if (RootComponent->Mobility == EComponentMobility::Static)
				{
					return LOCTEXT("ComponentMobility_Static", "Static");
				}
				else if (RootComponent->Mobility == EComponentMobility::Stationary)
				{
					return LOCTEXT("ComponentMobility_Stationary", "Stationary");
				}
				else if (RootComponent->Mobility == EComponentMobility::Movable)
				{
					return LOCTEXT("ComponentMobility_Movable", "Movable");
				}
			}
			else
			{
				return LOCTEXT("ComponentMobility_NoRoot", "No root component, unknown mobility");
			}
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SSubobject_RowWidget::GetInheritedLinkWidget()
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
    const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if(!Data)
	{
		return SNullWidget::NullWidget;
	}

	if (TSharedPtr<SSubobjectEditor> PinnedSubobjectEditor = SubobjectEditor.Pin())
	{
		if (const ISCSEditorUICustomization* UICustomization = PinnedSubobjectEditor->GetUICustomization())
		{
			TSharedRef<SSubobjectEditor> SubobjectEditorRef = PinnedSubobjectEditor.ToSharedRef();
			if (TSharedPtr<SWidget> ControlsWidget = UICustomization->GetControlsWidget(SubobjectEditorRef, *Data))
			{
				return ControlsWidget.ToSharedRef();
			}
		}
	}
	
	// Native components are inherited and have a gray hyperlink to their C++ class
	if(Data->IsNativeComponent())
	{
		static const FText NativeCppLabel = LOCTEXT("NativeCppInheritedLabel", "Edit in C++");

		return SNew(SHyperlink)
			.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
			.OnNavigate(this, &SSubobject_RowWidget::OnEditNativeCppClicked)
			.Text(NativeCppLabel)
			.ToolTipText(FText::Format(LOCTEXT("GoToCode_ToolTip", "Click to open this source file in {0}"), FSourceCodeNavigation::GetSelectedSourceCodeIDE()))
			.Visibility(this, &SSubobject_RowWidget::GetEditNativeCppVisibility);
	}
	// If the subobject is inherited and not native then it must be from a blueprint
	else if(Data->IsInstancedInheritedComponent() || Data->IsBlueprintInheritedComponent())
	{
		if(const UBlueprint* BP = Data->GetBlueprint())
		{
			static const FText InheritedBPLabel = LOCTEXT("InheritedBpLabel", "Edit in Blueprint");
			return SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
				.OnNavigate(this, &SSubobject_RowWidget::OnEditBlueprintClicked)
				.Text(InheritedBPLabel)
				.ToolTipText(LOCTEXT("EditBlueprint_ToolTip", "Click to edit the blueprint"))
				.Visibility(this, &SSubobject_RowWidget::GetEditBlueprintVisibility);
		}
	}

	// Non-inherited subobjects shouldn't show anything! 
	return SNullWidget::NullWidget;
}

FText SSubobject_RowWidget::GetObjectContextText() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
    const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	
	// We want to display (Self) or (Instance) only if the data is an actor
    if (Data && Data->IsActor())
    {
	   return Data->GetDisplayNameContextModifiers(GetDefault<UEditorStyleSettings>()->bShowNativeComponentNames);
    }
	return FText::GetEmpty();
}

void SSubobject_RowWidget::OnEditBlueprintClicked()
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
    const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if(!Data)
	{
		return;
	}

	// Bring focus to or open the blueprint asset
	if(UBlueprint* Blueprint = Data->GetBlueprint())
	{
		if (Blueprint->UbergraphPages.Num() > 0)
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Blueprint->GetLastEditedUberGraph());
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
		}
	}
}

EVisibility SSubobject_RowWidget::GetEditBlueprintVisibility() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;

	if (!Data)
	{
		return EVisibility::Collapsed;
	}

	if (const UBlueprint* BP = Data->GetBlueprint())
	{
		return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->IsAssetEditable(BP) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

EVisibility SSubobject_RowWidget::GetEditNativeCppVisibility() const
{
	return ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SSubobject_RowWidget::OnEditNativeCppClicked()
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
    const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if(!Data)
	{
		return;
	}

	const FName VariableName = Data->GetVariableName();
	const UClass* ParentClass = nullptr;
	// If the property came from a blueprint then we can get it from generated class
	if(UBlueprint* Blueprint = Data->GetBlueprint())
	{
		ParentClass = Blueprint ? *Blueprint->SkeletonGeneratedClass : nullptr;
	}
	// Otherwise we have get the class from the outer and look for a property on it
	else if(const FSubobjectData* ParentData = Data->GetRootSubobject().GetData())
	{
		const UObject* ParentObj = ParentData->GetObject();
		ParentClass = ParentObj ? ParentObj->GetClass() : nullptr;
	}
	
	const FProperty* VariableProperty = ParentClass ? FindFProperty<FProperty>(ParentClass, VariableName) : nullptr;
	FSourceCodeNavigation::NavigateToProperty(VariableProperty);
}

FString SSubobject_RowWidget::GetDocumentationLink() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	
	if (Data && (NodePtr == SubobjectEditor.Pin()->GetSceneRootNode() || Data->IsInheritedComponent()))
	{
		return TEXT("Shared/Editors/BlueprintEditor/ComponentsMode");
	}

	return TEXT("");
}

FString SSubobject_RowWidget::GetDocumentationExcerptName() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;

	if(NodePtr && Data)
	{
		if (NodePtr == SubobjectEditor.Pin()->GetSceneRootNode())
		{
			return TEXT("RootComponent");
		}
		else if (Data->IsNativeComponent())
		{
			return TEXT("NativeComponents");
		}
		else if (Data->IsInheritedComponent())
		{
			return TEXT("InheritedComponents");
		}	
	}

	return TEXT("");
}

FText SSubobject_RowWidget::GetMobilityToolTipText() const
{
	const FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	
	if (Data)
	{
		return Data->GetMobilityToolTipText();
	}

	return LOCTEXT("ErrorNoMobilityTooltip", "Invalid component");
}

FSlateBrush const* SSubobject_RowWidget::GetMobilityIconImage() const
{
	if (FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr())
	{
		if(FSubobjectData* Data = NodePtr->GetDataSource())
		{
			if (const USceneComponent* SceneComponentTemplate = Cast<USceneComponent>(Data->GetComponentTemplate()))
			{
				if (SceneComponentTemplate->Mobility == EComponentMobility::Movable)
				{
					return FAppStyle::GetBrush(TEXT("ClassIcon.MovableMobilityIcon"));
				}
				else if (SceneComponentTemplate->Mobility == EComponentMobility::Stationary)
				{
					return FAppStyle::GetBrush(TEXT("ClassIcon.StationaryMobilityIcon"));
				}
				// static components don't get an icon (because static is the most common
				// mobility type, and we'd like to keep the icon clutter to a minimum)
			}
		}	
	}

	return nullptr;
}

FText SSubobject_RowWidget::GetIntroducedInToolTipText() const
{
	FText IntroducedInTooltip = LOCTEXT("IntroducedInThisBPTooltip", "this class");
	if (const FSubobjectData* TreeNode = SubobjectPtr->GetDataSource())
	{
		return TreeNode->GetIntroducedInToolTipText();
	}

	return IntroducedInTooltip;
}

FText SSubobject_RowWidget::GetComponentAddSourceToolTipText() const
{
	FText NodeType;
	
	if (const FSubobjectData* TreeNode = SubobjectPtr->GetDataSource())
	{
		if (TreeNode->IsInheritedComponent())
		{
			if (TreeNode->IsNativeComponent())
			{
				NodeType = LOCTEXT("InheritedNativeComponent", "Inherited (C++)");
			}
			else
			{
				NodeType = LOCTEXT("InheritedBlueprintComponent", "Inherited (Blueprint)");
			}
		}
		else
		{
			if (TreeNode->IsInstancedComponent())
			{
				NodeType = LOCTEXT("ThisInstanceAddedComponent", "This actor instance");
			}
			else
			{
				NodeType = LOCTEXT("ThisBlueprintAddedComponent", "This Blueprint");
			}
		}
	}

	return NodeType;
}

FText SSubobject_RowWidget::GetComponentEditorOnlyTooltipText() const
{
	const FSubobjectData* TreeNode = SubobjectPtr->GetDataSource();
	return TreeNode ? TreeNode->GetComponentEditorOnlyTooltipText() : LOCTEXT("ComponentEditorOnlyFalse", "False");
}

FText SSubobject_RowWidget::GetNativeComponentNameToolTipText() const
{
	const FSubobjectData* TreeNode = SubobjectPtr->GetDataSource();
	const UActorComponent* Template = TreeNode ? TreeNode->GetComponentTemplate() : nullptr;

	if (Template)
	{
		return FText::FromName(Template->GetFName());
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText SSubobject_RowWidget::GetActorDisplayText() const
{
	const FSubobjectData* TreeNode = SubobjectPtr->GetDataSource();

	if (TreeNode)
	{
		return TreeNode->GetActorDisplayText();
	}
	
	return FText::GetEmpty();
}

void SSubobject_RowWidget::OnNameTextCommit(const FText& InNewName, ETextCommit::Type InTextCommit)
{
	// Ask the subsystem to rename this
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	if (ensure(NodePtr))
	{
		// If there is already an ongoing transaction then use that, otherwise make a
		// new transaction for renaming the component
		TUniquePtr<FScopedTransaction> Transaction;
		NodePtr->GetOngoingCreateTransaction(Transaction);

		// If a 'create' transaction is opened, the rename will be folded into it and will be invisible to the
		// 'undo' as create + give a name is really just one operation from the user point of view.
		FScopedTransaction TransactionContext(LOCTEXT("RenameComponentVariable", "Rename Component Variable"));

		// Ask the subsystem to verify this rename option for us
		USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
		check(System);
		const bool bSuccess = System->RenameSubobject(NodePtr->GetDataHandle(), InNewName);
		
		Transaction.Reset();
	}
}

bool SSubobject_RowWidget::OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage)
{
	const FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	if (ensure(NodePtr))
	{
		// Ask the subsystem to verify this rename option for us
		const USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
		check(System);
		return System->IsValidRename(NodePtr->GetDataHandle(), InNewText, OutErrorMessage);
	}
	
	return false;
}

bool SSubobject_RowWidget::IsReadOnly() const
{
	FSubobjectEditorTreeNodePtrType NodePtr = GetSubobjectPtr();
	const FSubobjectData* Data = NodePtr ? NodePtr->GetDataSource() : nullptr;
	if (Data)
	{
		return !Data->CanRename() || (SubobjectEditor.IsValid() && !SubobjectEditor.Pin()->IsEditingAllowed());
	}

	// If there is no valid data then default to read only
	return true;
}

////////////////////////////////////////////////
// SSubobjectEditor

FMenuBuilder SSubobjectEditor::CreateMenuBuilder()
{
	// Menu builder for editing a blueprint
	FMenuBuilder EditBlueprintMenuBuilder(true, nullptr);

	EditBlueprintMenuBuilder.BeginSection(
		NAME_None, LOCTEXT("EditBlueprintMenu_ExistingBlueprintHeader", "Existing Blueprint"));

	EditBlueprintMenuBuilder.AddMenuEntry
	(
		LOCTEXT("OpenBlueprintEditor", "Open Blueprint Editor"),
		LOCTEXT("OpenBlueprintEditor_ToolTip", "Opens the blueprint editor for this asset"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSubobjectEditor::OnOpenBlueprintEditor, /*bForceCodeEditing=*/ false))
	);

	EditBlueprintMenuBuilder.AddMenuEntry
	(
		LOCTEXT("OpenBlueprintEditorScriptMode", "Add or Edit Script"),
		LOCTEXT("OpenBlueprintEditorScriptMode_ToolTip",
		        "Opens the blueprint editor for this asset, showing the event graph"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::OnOpenBlueprintEditor, /*bForceCodeEditing=*/ true))
	);

	return EditBlueprintMenuBuilder;
}

void SSubobjectEditor::InitScopeRow(FStringView Label)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Scope;
	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		ScopeRow = Storage->AddScopeRow(Label);
		Storage->SetParentScope(ScopeRow, GetCurrentScope());
	}
}

SSubobjectEditor::~SSubobjectEditor()
{
	using namespace UE::Editor::DataStorage;
	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		if (Storage->IsRowAvailable(ScopeRow))
		{
			Storage->RemoveScopeRow(ScopeRow);
		}
	}
	ScopeRow = InvalidRowHandle;
}

void SSubobjectEditor::ConstructTreeWidget()
{
	TSharedPtr<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(SubobjectTree_ColumnName_ComponentClass)
		  .DefaultLabel(LOCTEXT("Class", "Class"))
		  .FillWidth(4);

	TreeWidget = SNew(SSubobjectEditorDragDropTree)
		.SubobjectEditor(this)
		.ToolTipText(LOCTEXT("DropAssetToAddComponent", "Drop asset here to add a component."))
		.TreeItemsSource(&RootNodes)
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SSubobjectEditor::MakeTableRowWidget)
		.OnGetChildren(this, &SSubobjectEditor::OnGetChildrenForTree)
		.OnSetExpansionRecursive(this, &SSubobjectEditor::SetItemExpansionRecursive)
		.OnSelectionChanged(this, &SSubobjectEditor::OnTreeSelectionChanged)
		.OnContextMenuOpening(this, &SSubobjectEditor::CreateContextMenu)
		.OnItemScrolledIntoView(this, &SSubobjectEditor::OnItemScrolledIntoView)
		.OnMouseButtonDoubleClick(this, &SSubobjectEditor::HandleItemDoubleClicked)
		.ClearSelectionOnClick(ClearSelectionOnClick())
		.OnTableViewBadState(this, &SSubobjectEditor::DumpTree)
		.HeaderRow
	    (
			HeaderRow
	    );

	HeaderRow->SetVisibility(EVisibility::Collapsed);
}

void SSubobjectEditor::CreateCommandList()
{
	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(FGenericCommands::Get().Cut,
       FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::CutSelectedNodes),
       FCanExecuteAction::CreateSP(this, &SSubobjectEditor::CanCutNodes))
    );

	CommandList->MapAction(FGenericCommands::Get().Copy,
		FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &SSubobjectEditor::CanCopyNodes))
	);

	CommandList->MapAction(FGenericCommands::Get().Paste,
	    FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::PasteNodes),
	    FCanExecuteAction::CreateSP(this, &SSubobjectEditor::CanPasteNodes))
	);

	CommandList->MapAction(FGenericCommands::Get().Duplicate,
	    FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::OnDuplicateComponent),
	    FCanExecuteAction::CreateSP(this, &SSubobjectEditor::CanDuplicateComponent))
	);

	CommandList->MapAction(FGenericCommands::Get().Delete,
	    FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::OnDeleteNodes),
		FCanExecuteAction::CreateSP(this, &SSubobjectEditor::CanDeleteNodes))
	);

	CommandList->MapAction(FGenericCommands::Get().Rename,
	    FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::OnRenameComponent),
	    FCanExecuteAction::CreateSP(this, &SSubobjectEditor::CanRenameComponent))
	);

	CommandList->MapAction(FGraphEditorCommands::Get().GetFindReferences(),
		FUIAction(FExecuteAction::CreateSP(this, &SSubobjectEditor::OnFindReferences, false, EGetFindReferenceSearchStringFlags::Legacy) )
	);
	
	CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByNameLocal,
		FUIAction( FExecuteAction::CreateSP( this, &SSubobjectEditor::OnFindReferences, false, EGetFindReferenceSearchStringFlags::None) )
	);
	
	CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByNameGlobal,
		FUIAction( FExecuteAction::CreateSP( this, &SSubobjectEditor::OnFindReferences, true, EGetFindReferenceSearchStringFlags::None) )
	);
	
	CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByClassMemberLocal,
		FUIAction( FExecuteAction::CreateSP( this, &SSubobjectEditor::OnFindReferences, false, EGetFindReferenceSearchStringFlags::UseSearchSyntax) )
	);

	CommandList->MapAction( FGraphEditorCommands::Get().FindReferencesByClassMemberGlobal,
		FUIAction( FExecuteAction::CreateSP( this, &SSubobjectEditor::OnFindReferences, true, EGetFindReferenceSearchStringFlags::UseSearchSyntax) )
	);
}

bool SSubobjectEditor::IsComponentSelected(const UPrimitiveComponent* PrimComponent) const
{
	check(PrimComponent);

	if (TreeWidget.IsValid())
	{
		FSubobjectEditorTreeNodePtrType NodePtr = FindSlateNodeForObject(PrimComponent, false);
		if (NodePtr.IsValid())
		{
			return TreeWidget->IsItemSelected(NodePtr);
		}
		else
		{
			UChildActorComponent* PossiblySelectedComponent = nullptr;
			AActor* ComponentOwner = PrimComponent->GetOwner();
			while (ComponentOwner->IsChildActor())
			{
				PossiblySelectedComponent = ComponentOwner->GetParentComponent();
				ComponentOwner = ComponentOwner->GetParentActor();
			}

			if (PossiblySelectedComponent)
			{
				NodePtr = FindSlateNodeForObject(PossiblySelectedComponent, false);
				if (NodePtr.IsValid())
				{
					return TreeWidget->IsItemSelected(NodePtr);
				}
			}
		}
	}
	
	return false;
}

void SSubobjectEditor::SetSelectionOverride(UPrimitiveComponent* PrimComponent) const
{
	PrimComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateSP(this, &SSubobjectEditor::IsComponentSelected);
	PrimComponent->PushSelectionToProxy();
}

TSharedRef<ITableRow> SSubobjectEditor::MakeTableRowWidget(FSubobjectEditorTreeNodePtrType InNodePtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	FGraphNodeMetaData TagMeta(TEXT("TableRow"));
	const UActorComponent* Template = InNodePtr && !InNodePtr->IsSeperator()
		                                  ? InNodePtr->GetDataSource()->GetComponentTemplate()
		                                  : nullptr;
	if (Template)
	{
		TagMeta.FriendlyName = FString::Printf(TEXT("TableRow,%s,0"), *Template->GetReadableName());
	}

	return SNew(SSubobject_RowWidget, SharedThis(this), InNodePtr, OwnerTable)
		.AddMetaData<FTutorialMetaData>(TagMeta);
}

EVisibility SSubobjectEditor::GetComponentsTreeVisibility() const
{
	UObject* ObjectContextPtr = GetObjectContext();
	return (UICustomization.IsValid() && UICustomization->HideComponentsTree(MakeArrayView(&ObjectContextPtr, 1)))
		       ? EVisibility::Collapsed
		       : EVisibility::Visible;
}

void SSubobjectEditor::OnGetChildrenForTree(FSubobjectEditorTreeNodePtrType InNodePtr, TArray<FSubobjectEditorTreeNodePtrType>& OutChildren)
{
	if (InNodePtr)
	{
		const TArray<FSubobjectEditorTreeNodePtrType>& Children = InNodePtr->GetChildren();
		OutChildren.Reserve(Children.Num());

		if (GetComponentTypeFilterToApply() || !GetFilterText().IsEmpty())
		{
			for(FSubobjectEditorTreeNodePtrType Child : Children)
			{
				if (!Child->IsFlaggedForFiltration())
				{
					OutChildren.Add(Child);
				}
			}
		}
		else
		{
			OutChildren = Children;	
		}
	}
	else
	{
		OutChildren.Empty();
	}
}

TArray<FSubobjectEditorTreeNodePtrType> SSubobjectEditor::GetSelectedNodes() const
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedTreeNodes = TreeWidget->GetSelectedItems();

	struct FCompareSelectedSubobjectEditorTreeNodes
	{
		FORCEINLINE bool operator()(const FSubobjectEditorTreeNodePtrType& A,
		                            const FSubobjectEditorTreeNodePtrType& B) const
		{
			return B.IsValid() && B->IsAttachedTo(A);
		}
	};

	// Ensure that nodes are ordered from parent to child (otherwise they are sorted in the order that they were selected)
	SelectedTreeNodes.Sort(FCompareSelectedSubobjectEditorTreeNodes());

	return SelectedTreeNodes;
}

TArray<FSubobjectDataHandle> SSubobjectEditor::GetSelectedHandles() const
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = GetSelectedNodes();
	TArray<FSubobjectDataHandle> OutHandles;
	for(FSubobjectEditorTreeNodePtrType& Ptr : SelectedNodes)
	{
		OutHandles.Add(Ptr->GetDataHandle());
	}
	return OutHandles;
}

FSubobjectEditorTreeNodePtrType SSubobjectEditor::GetSceneRootNode() const
{
	// Get the first scene component in the root nodes
	if(const AActor* ActorContext = Cast<AActor>(GetObjectContext()))
	{
		if(const USceneComponent* RootComp = ActorContext->GetRootComponent())
		{
			TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
			USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
			
			return FindSlateNodeForHandle(System->FindSceneRootForSubobject(GetObjectContextHandle()));
		}
	}

	return nullptr;
}

EVisibility SSubobjectEditor::GetDesiredVisibility() const
{
	EVisibility CustomizationVisibility = EVisibility::Collapsed;
	UE::Editor::FSubobjectEditorContext Context;
	Context.ObjectContext = GetObjectContext();
	Context.SelectionSet = SelectionSet;
	if (UICustomization && UICustomization->OverrideDesiredVisibility(Context, CustomizationVisibility))
	{
		return CustomizationVisibility;
	}

	return Context.ObjectContext ? EVisibility::Visible : EVisibility::Collapsed;
}

void SSubobjectEditor::UpdateSelectionFromNodes(const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes)
{
	bUpdatingSelection = true;

	// Push our scope row so that downstream code (e.g. property visibility checks
	// in the details panel) sees the correct editing scope via GetCurrentScope()
	UE::Editor::DataStorage::Scope::FPushScopeGuard PushScopeGuard(ScopeRow);

	// Notify that the selection has updated
	OnSelectionUpdated.ExecuteIfBound(SelectedNodes);

	bUpdatingSelection = false;
}

void SSubobjectEditor::OnTreeSelectionChanged(FSubobjectEditorTreeNodePtrType InSelectedNodePtr, ESelectInfo::Type SelectInfo)
{
	UpdateSelectionFromNodes(TreeWidget->GetSelectedItems());
}

void SSubobjectEditor::HandleItemDoubleClicked(FSubobjectEditorTreeNodePtrType InItem)
{
	OnItemDoubleClicked.ExecuteIfBound(InItem);
}

void SSubobjectEditor::OnFindReferences(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags)
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = TreeWidget->GetSelectedItems();
	if (SelectedNodes.Num() == 1)
	{
		TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(GetBlueprint());
		FSubobjectData* Data = SelectedNodes[0]->GetDataSource();
		if (FoundAssetEditor.IsValid() && Data)
		{
			const FString VariableName = Data->GetVariableName().ToString();

			FMemberReference MemberReference;
			MemberReference.SetSelfMember(*VariableName);
			const FString SearchTerm = EnumHasAnyFlags(Flags, EGetFindReferenceSearchStringFlags::UseSearchSyntax) ? MemberReference.GetReferenceSearchString(GetBlueprint()->SkeletonGeneratedClass) : FString::Printf(TEXT("\"%s\""), *VariableName);

			TSharedRef<IBlueprintEditor> BlueprintEditor = StaticCastSharedRef<IBlueprintEditor>(
				FoundAssetEditor.ToSharedRef());
			
			const bool bSetFindWithinBlueprint = !bSearchAllBlueprints;
			BlueprintEditor->SummonSearchUI(bSetFindWithinBlueprint, SearchTerm);
		}
	}
}

void SSubobjectEditor::SelectRoot()
{
	if (RootNodes.Num() > 0)
	{
		TreeWidget->SetSelection(RootNodes[0]);
	}
}

FSlateColor SSubobjectEditor::GetColorTintForIcon(FSubobjectEditorTreeNodePtrType Node) const
{
	return FSlateColor::UseForeground();
}

void SSubobjectEditor::SelectNodeFromHandle(const FSubobjectDataHandle& InHandle, bool bIsCntrlDown)
{
	SelectNode(FindSlateNodeForHandle(InHandle), bIsCntrlDown);
}

void SSubobjectEditor::SelectNode(FSubobjectEditorTreeNodePtrType InNodeToSelect, bool bIsCntrlDown)
{
	if (TreeWidget.IsValid() && InNodeToSelect.IsValid())
	{
		if (!bIsCntrlDown)
		{
			TreeWidget->SetSelection(InNodeToSelect);
		}
		else
		{
			TreeWidget->SetItemSelection(InNodeToSelect, !TreeWidget->IsItemSelected(InNodeToSelect));
		}
	}
}

EVisibility SSubobjectEditor::GetComponentsFilterBoxVisibility() const
{
	UObject* ObjectContextPtr = GetObjectContext();
	return (UICustomization.IsValid() && UICustomization->HideComponentsFilterBox(MakeArrayView(&ObjectContextPtr, 1)))
		       ? EVisibility::Collapsed
		       : EVisibility::Visible;
}

void SSubobjectEditor::OnFilterTextChanged(const FText& InFilterText)
{
	struct OnFilterTextChanged_Inner
	{
		static FSubobjectEditorTreeNodePtrType ExpandToFilteredChildren(SSubobjectEditor* SubobjectEditor,
		                                                                FSubobjectEditorTreeNodePtrType TreeNode)
		{
			FSubobjectEditorTreeNodePtrType NodeToFocus;

			const TArray<FSubobjectEditorTreeNodePtrType>& Children = TreeNode->GetChildren();
			// iterate backwards so we select from the top down
			for (int32 ChildIndex = Children.Num() - 1; ChildIndex >= 0; --ChildIndex)
			{
				const FSubobjectEditorTreeNodePtrType& Child = Children[ChildIndex];
				// Don't attempt to focus a separator or filtered node
				if (!Child->IsSeperator() && !Child->IsFlaggedForFiltration())
				{
					SubobjectEditor->SetNodeExpansionState(TreeNode, /*bIsExpanded =*/true);
					NodeToFocus = ExpandToFilteredChildren(SubobjectEditor, Child);
				}
			}

			if (!NodeToFocus.IsValid() && !TreeNode->IsFlaggedForFiltration())
			{
				NodeToFocus = TreeNode;
			}
			return NodeToFocus;
		}
	};

	FSubobjectEditorTreeNodePtrType NewSelection;
	// iterate backwards so we select from the top down
	for (int32 ComponentIndex = RootNodes.Num() - 1; ComponentIndex >= 0; --ComponentIndex)
	{
		FSubobjectEditorTreeNodePtrType Node = RootNodes[ComponentIndex];

		bool bIsRootVisible = !RefreshFilteredState(Node, true);
		TreeWidget->SetItemExpansion(Node, bIsRootVisible);
		if (bIsRootVisible)
		{
			if (!GetFilterText().IsEmpty())
			{
				NewSelection = OnFilterTextChanged_Inner::ExpandToFilteredChildren(this, Node);
			}
		}
	}

	if (!NewSelection.IsValid() && RootNodes.Num() > 0)
	{
		NewSelection = RootNodes[0];
	}

	if (NewSelection.IsValid() && !TreeWidget->IsItemSelected(NewSelection))
	{
		SelectNode(NewSelection, /*IsCntrlDown =*/false);
	}

	UpdateTree(/*bRegenerateTreeNodes =*/false);
}

void SSubobjectEditor::OnItemScrolledIntoView(FSubobjectEditorTreeNodePtrType InItem, const TSharedPtr<ITableRow>& InWidget)
{
	if(DeferredRenameRequest.IsValid())
	{
		const FSubobjectDataHandle& ItemHandle = InItem->GetDataHandle();
		if(DeferredRenameRequest == ItemHandle)
		{
			DeferredRenameRequest = FSubobjectDataHandle::InvalidHandle;
			
			// Transfer responsibility to end the 'create + give initial name' transaction to the tree item if such transaction is ongoing.
			// We want to request a rename from the inline textbox on the associated handle
			InItem->SetOngoingCreateTransaction(MoveTemp(DeferredOngoingCreateTransaction));
			// Notify any listeners that this node will be renamed (this will trigger the inline text boxes on the
			// row widgets to turn to edit mode)
			InItem->GetRenameRequestedDelegate().ExecuteIfBound();
		}
	}
}

void SSubobjectEditor::UpdateTree(bool bRegenerateTreeNodes /* = true */)
{
	check(TreeWidget.IsValid());

	// Early exit if we're deferring tree updates
	if (!bAllowTreeUpdates)
	{
		return;
	}

	if (bRegenerateTreeNodes)
	{
		// Obtain the list of selected items
		TArray<FSubobjectEditorTreeNodePtrType> SelectedTreeNodes = GetSelectedNodes();

		// Clear the current tree
		if (SelectedTreeNodes.Num() != 0)
		{
			TreeWidget->ClearSelection();
		}
		// recursively purge nodes because their ownership is a mess - and the expected
		// cascaded delete does not happen automatically:
		ForEachNode([](FSubobjectEditorTreeNode& Tree) 
			{
				Tree.DisposeUnsafe();
			}
		);
		RootNodes.Empty();

		if (GetDesiredVisibility() == EVisibility::Visible)
		{
			ensureMsgf(FModuleManager::Get().LoadModule("SubobjectDataInterface"), TEXT("The Subobject Data Interface module is required."));

			USubobjectDataSubsystem* DataSubsystem = USubobjectDataSubsystem::Get();
			check(DataSubsystem);

			TArray<FSubobjectDataHandle> SubobjectData;

			UE::Editor::FSubobjectEditorContext Params;
			Params.ObjectContext = GetObjectContext();
			Params.SelectionSet = SelectionSet;
			DataSubsystem->GatherSubobjectData(Params, SubobjectData);

			if (UICustomization)
			{
				UICustomization->FilterSubobjectData(SubobjectData);
				UICustomization->SortSubobjectData(SubobjectData);
			}

			FSubobjectEditorTreeNodePtrType SeperatorNode;
			TMap<FSubobjectDataHandle, FSubobjectEditorTreeNodePtrType> AddedNodes;

			// By default, root node will always be expanded. If possible we will restore the collapsed state later.
			if (SubobjectData.Num() > 0)
			{
				FSubobjectEditorTreeNodePtrType Node = MakeShareable<FSubobjectEditorTreeNode>(
					new FSubobjectEditorTreeNode(SubobjectData[0]));
				RootNodes.Add(Node);
				AddedNodes.Add(Node->GetDataHandle(), Node);
				CachedRootHandle = Node->GetDataHandle();

				TreeWidget->SetItemExpansion(Node, true);
				TreeWidget->SetItemExpansion(SeperatorNode, true);

				RefreshFilteredState(Node, false);
			}

			// Create slate nodes for each subobject
			for (FSubobjectDataHandle& Handle : SubobjectData)
			{
				// Do we have a slate node for this handle already? If not, then we need to make one
				FSubobjectEditorTreeNodePtrType NewNode = SSubobjectEditor::FindOrCreateSlateNodeForHandle(Handle, AddedNodes);

				FSubobjectData* Data = Handle.GetData();

				const FSubobjectDataHandle& ParentHandle = Data->GetParentHandle();

				// Have parent? 
				if (ParentHandle.IsValid())
				{
					// Get the parent node for this subobject
					FSubobjectEditorTreeNodePtrType ParentNode = SSubobjectEditor::FindOrCreateSlateNodeForHandle(ParentHandle, AddedNodes);

					check(ParentNode);
					ParentNode->AddChild(NewNode);
					TreeWidget->SetItemExpansion(ParentNode, true);

					const bool bFilteredOut = RefreshFilteredState(NewNode, false);

					// Add a separator after the default scene root, but only it is not filtered out and if there are more items below it
					if (!bFilteredOut &&
						Data->IsDefaultSceneRoot() &&
						Data->IsInheritedComponent() && 
						SubobjectData.Find(Handle) < SubobjectData.Num() - 1)
					{
						SeperatorNode = MakeShareable<FSubobjectEditorTreeNode>(
							new FSubobjectEditorTreeNode(FSubobjectDataHandle::InvalidHandle, /** bIsSeperator */true));
						AddedNodes.Add(SeperatorNode->GetDataHandle(), SeperatorNode);
						ParentNode->AddChild(SeperatorNode);
					}

				}
				TreeWidget->SetItemExpansion(NewNode, true);
			}

			RestoreSelectionState(SelectedTreeNodes);

			// If we have a pending deferred rename request, redirect it to the new tree node
			if (DeferredRenameRequest.IsValid())
			{
				FSubobjectEditorTreeNodePtrType NodeToRenamePtr = FindSlateNodeForHandle(DeferredRenameRequest);
				if (NodeToRenamePtr.IsValid())
				{
					TreeWidget->RequestScrollIntoView(NodeToRenamePtr);
				}
			}
		}
	}
	
	TreeWidget->RequestTreeRefresh();
}

FSubobjectEditorTreeNodePtrType SSubobjectEditor::FindOrCreateSlateNodeForHandle(const FSubobjectDataHandle& InHandle, TMap<FSubobjectDataHandle, FSubobjectEditorTreeNodePtrType>& ExistingNodes)
{
	// If we have already created a tree node for this handle, great! Return that.
	if (const FSubobjectEditorTreeNodePtrType* Found = ExistingNodes.Find(InHandle))
	{
		return *Found;
	}

	// Otherwise we haven't created this yet and we need to make a new one!
	FSubobjectEditorTreeNodePtrType NewNode = MakeShareable<FSubobjectEditorTreeNode>(new FSubobjectEditorTreeNode(InHandle));
	ExistingNodes.Add(InHandle, NewNode);

	return NewNode;
}

void SSubobjectEditor::RefreshSelectionDetails()
{
	UpdateSelectionFromNodes(TreeWidget->GetSelectedItems());
}

void SSubobjectEditor::PrePotentialTargetChange()
{
	if (UICustomization)
	{
		UICustomization->PrePotentialTargetChange(*this);
	}
}

void SSubobjectEditor::ClearSelection()
{
	if (bUpdatingSelection == false)
	{
		check(TreeWidget.IsValid());
		TreeWidget->ClearSelection();
	}
}

void SSubobjectEditor::SelectDefaultSubobject()
{
	if (UICustomization)
	{
		if (UICustomization->SelectDefaultSubobject(*this))
		{
			return;
		}
	}
	SelectRoot();
}

void SSubobjectEditor::RestoreSelectionState(TArray<FSubobjectEditorTreeNodePtrType>& SelectedTreeNodes, bool bFallBackToVariableName)
{
	if (SelectedTreeNodes.Num() > 0)
	{
		// If there is only one item selected, imitate user selection to preserve navigation
		ESelectInfo::Type SelectInfo = SelectedTreeNodes.Num() == 1 ? ESelectInfo::OnMouseClick : ESelectInfo::Direct;

		// Restore the previous selection state on the new tree nodes
		for (int i = 0; i < SelectedTreeNodes.Num(); ++i)
		{
			if (RootNodes.Num() > 0 && SelectedTreeNodes[i] == RootNodes[0])
			{
				TreeWidget->SetItemSelection(RootNodes[0], true, SelectInfo);
			}
			else
			{
				FSubobjectDataHandle CurrentNodeDataHandle = SelectedTreeNodes[i]->GetDataHandle();
				FSubobjectEditorTreeNodePtrType NodeToSelectPtr = FindSlateNodeForHandle(CurrentNodeDataHandle);

				// If we didn't find something for this exact handle, fall back to just search for something
				// with the same variable name. This helps to still preserve selection across re-compiles of a class.
				if (!NodeToSelectPtr.IsValid() && CurrentNodeDataHandle.IsValid() && bFallBackToVariableName)
				{
					NodeToSelectPtr = FindSlateNodeForVariableName(CurrentNodeDataHandle.GetSharedDataPtr()->GetVariableName());
				}

				if (NodeToSelectPtr.IsValid())
				{
					TreeWidget->SetItemSelection(NodeToSelectPtr, true, SelectInfo);
				}
			}

			//if (GetEditorMode() != EComponentEditorMode::BlueprintSCS)
			//{
			//	TArray<FSubobjectEditorTreeNodePtrType> NewSelectedTreeNodes = TreeWidget->GetSelectedItems();
			//	if (NewSelectedTreeNodes.Num() == 0)
			//	{
			//		TreeWidget->SetItemSelection(RootNodes[0], true);
			//	}
			//}
		}
	}
}

FReply SSubobjectEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SSubobjectEditor::SetSelectionSet(UTypedElementSelectionSet* InSelectionSet)
{
	SelectionSet = InSelectionSet;
}

UTypedElementSelectionSet* SSubobjectEditor::GetSelectionSet() const
{
	return SelectionSet.Get();
}

FReply SSubobjectEditor::TryHandleAssetDragDropOperation(const FDragDropEvent& DragDropEvent)
{
	return TryHandleAssetDragDropOperation(DragDropEvent, nullptr);
}

FReply SSubobjectEditor::TryHandleAssetDragDropOperation(const FDragDropEvent& InDragDropEvent, FSubobjectEditorTreeNodePtrType InTargetItem)
{
	TSharedPtr<FDragDropOperation> Operation = InDragDropEvent.GetOperation();
	if (Operation.IsValid() && (Operation->IsOfType<FExternalDragOperation>() || Operation->IsOfType<FAssetDragDropOp>()))
	{
		TArray< FAssetData > DroppedAssetData = AssetUtil::ExtractAssetDataFromDrag(Operation);
		const int32 NumAssets = DroppedAssetData.Num();

		if (NumAssets > 0)
		{
			GWarn->BeginSlowTask(LOCTEXT("LoadingAssets", "Loading Asset(s)"), true);
			bool bMarkBlueprintAsModified = false;

			for (int32 DroppedAssetIdx = 0; DroppedAssetIdx < NumAssets; ++DroppedAssetIdx)
			{
				const FAssetData& AssetData = DroppedAssetData[DroppedAssetIdx];

				if (!AssetData.IsAssetLoaded())
				{
					GWarn->StatusUpdate(DroppedAssetIdx, NumAssets, FText::Format(LOCTEXT("LoadingAsset", "Loading Asset {0}"), FText::FromName(AssetData.AssetName)));
				}
				
				UClass* AssetClass = AssetData.GetClass();
				UObject* Asset = AssetData.GetAsset();

				UBlueprint* BPClass = Cast<UBlueprint>(Asset);
				UClass* PotentialComponentClass = nullptr;
				UClass* PotentialActorClass = nullptr;

				if ((BPClass != nullptr) && (BPClass->GeneratedClass != nullptr))
				{
					if (BPClass->GeneratedClass->IsChildOf(UActorComponent::StaticClass()))
					{
						PotentialComponentClass = BPClass->GeneratedClass;
					}
					else if (BPClass->GeneratedClass->IsChildOf(AActor::StaticClass()))
					{
						PotentialActorClass = BPClass->GeneratedClass;
					}
				}
				else if (AssetClass && AssetClass->IsChildOf(UClass::StaticClass()))
				{
					UClass* AssetAsClass = CastChecked<UClass>(Asset);
					if (AssetAsClass->IsChildOf(UActorComponent::StaticClass()))
					{
						PotentialComponentClass = AssetAsClass;
					}
					else if (AssetAsClass->IsChildOf(AActor::StaticClass()))
					{
						PotentialActorClass = AssetAsClass;
					}
				}
				
				USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
				check(System);

				TUniquePtr<FScopedTransaction> AddTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("AddComponent", "Add Component"));

				FAddNewSubobjectParams NewComponentParams;
				
				// Attach to the target item if any
				if (InTargetItem)
				{
					NewComponentParams.ParentHandle = InTargetItem->GetDataHandle();
				}

				// Attach to the currently selected handle, if there are none then select the root object
				if (!NewComponentParams.ParentHandle.IsValid())
				{
					const TArray<FSubobjectDataHandle>& SelectedHandles = GetSelectedHandles();
					NewComponentParams.ParentHandle = SelectedHandles.IsEmpty() ? GetObjectContextHandle() : SelectedHandles[0];
				}

				NewComponentParams.BlueprintContext = ShouldModifyBPOnAssetDrop() ? GetBlueprint() : nullptr;

				FText FailReason;
				NewComponentParams.bSkipMarkBlueprintModified = true;
				const bool bSetFocusToNewItem = (DroppedAssetIdx == NumAssets - 1);// Only set focus to the last item created
				
				TSubclassOf<UActorComponent> MatchingComponentClassForAsset = FComponentAssetBrokerage::GetPrimaryComponentForAsset(AssetClass);
				if (MatchingComponentClassForAsset != nullptr)
				{
					NewComponentParams.NewClass = MatchingComponentClassForAsset;
					NewComponentParams.AssetOverride = Asset;
					bMarkBlueprintAsModified = true;
					System->AddNewSubobject(NewComponentParams, FailReason);
				}
				else if ((PotentialComponentClass != nullptr) && !PotentialComponentClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract | CLASS_NewerVersionExists))
				{
					if (PotentialComponentClass->HasMetaData(FBlueprintMetadata::MD_BlueprintSpawnableComponent))
					{
						NewComponentParams.NewClass = PotentialComponentClass;
						NewComponentParams.AssetOverride = nullptr;
						bMarkBlueprintAsModified = true;
						System->AddNewSubobject(NewComponentParams, FailReason);
					}
				}
				else if ((PotentialActorClass != nullptr) && !PotentialActorClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract | CLASS_NewerVersionExists | CLASS_NotPlaceable))
				{
					NewComponentParams.NewClass = UChildActorComponent::StaticClass();
					NewComponentParams.AssetOverride = PotentialActorClass;
					bMarkBlueprintAsModified = true;
					System->AddNewSubobject(NewComponentParams, FailReason);
				}
			}
			
			// Optimization: Only mark the blueprint as modified at the end
			if (bMarkBlueprintAsModified && ShouldModifyBPOnAssetDrop())
			{
				UBlueprint* Blueprint = GetBlueprint();
				check(Blueprint != nullptr && Blueprint->SimpleConstructionScript != nullptr);

				Blueprint->Modify();
				if (Blueprint->SimpleConstructionScript)
				{
					Blueprint->SimpleConstructionScript->SaveToTransactionBuffer();
				}

				bAllowTreeUpdates = true;
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			}

			UpdateTree();

			GWarn->EndSlowTask();			
		}
		
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SSubobjectEditor::SetUICustomization(TSharedPtr<ISCSEditorUICustomization> InUICustomization)
{
	UICustomization = InUICustomization;

	UpdateTree(true /*bRegenerateTreeNodes*/);
}

const ISCSEditorUICustomization* SSubobjectEditor::GetUICustomization() const
{
	return UICustomization.Get();
}

bool SSubobjectEditor::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void SSubobjectEditor::CutSelectedNodes()
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = GetSelectedNodes();
	const FScopedTransaction Transaction(SelectedNodes.Num() > 1
		                                     ? LOCTEXT("CutComponents", "Cut Components")
		                                     : LOCTEXT("CutComponent", "Cut Component"));

	FSubobjectEditorTreeNodePtrType PostCutSelection = RootNodes.Num() > 0 ? RootNodes[0] : nullptr;
	for(FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedNodes)
	{
		if(FSubobjectEditorTreeNodePtrType ParentNode = SelectedNode->GetParent())
		{
			if(!SelectedNodes.Contains(ParentNode))
			{
				PostCutSelection = ParentNode;
				break;
			}
		}
	}
	
	CopySelectedNodes();
	OnDeleteNodes();

	// Select the parent node of the thing that was just cut
	if(PostCutSelection.IsValid())
	{
		TreeWidget->SetSelection(PostCutSelection);
	}
}

bool SSubobjectEditor::CanCopyNodes() const
{
	TArray<FSubobjectDataHandle> SelectedHandles = GetSelectedHandles();
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		return System->CanCopySubobjects(SelectedHandles);
	}
	return false;
}

bool SSubobjectEditor::CanPasteNodes() const
{
	if(!IsEditingAllowed())
	{
		return false;
	}
	
	if(FSubobjectEditorTreeNodePtrType SceneRoot = GetSceneRootNode())
	{
		if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
		{
			return System->CanPasteSubobjects(SceneRoot->GetDataHandle());
		}
	}

	return false;
}

bool SSubobjectEditor::ShouldClearFocusOnDuplicate(TSharedPtr<SWidget> InCurrentlyFocusedWidget)
{
	TSharedPtr<SWidget> CurrentWidget = InCurrentlyFocusedWidget;
	while (CurrentWidget)
	{
		for (const TSharedRef<FTagMetaData>& TagMetaData : CurrentWidget->GetAllMetaData<FTagMetaData>())
		{
			if (TagMetaData->Tag == SubobjectEditor_ClearFocusOnDuplicate)
			{
				return true;
			}
		}

		CurrentWidget = CurrentWidget->GetParentWidget();
	}

	return false;
}

bool SSubobjectEditor::CanDeleteNodes() const
{
	if(!IsEditingAllowed())
	{
		return false;
	}

	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = GetSelectedNodes();
	for (const FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedNodes)
	{
		if (!SelectedNode->IsValid() || !SelectedNode->CanDelete())
		{
			return false;
		}
	}
	
	return true;
}

bool SSubobjectEditor::CanDuplicateComponent() const
{
	if(!IsEditingAllowed())
	{
		return false;
	}
	
	TArray<FSubobjectDataHandle> SelectedHandles = GetSelectedHandles();
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		if (!RootNodes.IsEmpty())
		{
			return System->CanDuplicateSubobjects(RootNodes[0]->GetDataHandle(), SelectedHandles);
		}
	}

	return CanCopyNodes();
}

bool SSubobjectEditor::CanRenameComponent() const
{
	if (!IsEditingAllowed())
	{
		return false;
	}
	// In addition to certain node types, don't allow nodes within a child actor template's hierarchy to be renamed
	TArray<FSubobjectEditorTreeNodePtrType> SelectedItems = TreeWidget->GetSelectedItems();
	
	return SelectedItems.Num() == 1 && SelectedItems[0]->CanRename();
}

void SSubobjectEditor::OnRenameComponent()
{
	OnRenameComponent(nullptr);
}

void SSubobjectEditor::OnRenameComponent(TUniquePtr<FScopedTransaction> InComponentCreateTransaction)
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = GetSelectedNodes();

	// Should already be prevented from making it here.
	check(SelectedNodes.Num() == 1);

	DeferredRenameRequest = SelectedNodes[0]->GetDataHandle();

	// If this fails, something in the chain of responsibility failed to end the previous transaction.
	check(!DeferredOngoingCreateTransaction.IsValid());
	// If a 'create + give initial name' transaction is ongoing, take responsibility of ending it until the selected item is scrolled into view.
	DeferredOngoingCreateTransaction = MoveTemp(InComponentCreateTransaction); 
	
	TreeWidget->RequestScrollIntoView(SelectedNodes[0]);

	if (DeferredOngoingCreateTransaction.IsValid() && !PostTickHandle.IsValid())
	{
		// Ensure the item will be scrolled into view during the frame (See explanation in OnPostTick()).
		PostTickHandle = FSlateApplication::Get().OnPostTick().AddSP(this, &SSubobjectEditor::OnPostTick);
	}
}

void SSubobjectEditor::OnPostTick(float)
{
	// If a 'create + give initial name' is ongoing and the transaction ownership was not transferred during the frame it was requested, it is most likely because the newly
	// created item could not be scrolled into view (should say 'teleported', the scrolling is not animated). The tree view will not put the item in view if the there is
	// no space left to display the item. (ex a splitter where all the display space is used by the other component). End the transaction before starting a new frame. The user
	// will not be able to rename on creation, the widget is likely not in view and cannot be edited anyway.
	DeferredOngoingCreateTransaction.Reset();

	// The post tick event handler is not required anymore.
	FSlateApplication::Get().OnPostTick().Remove(PostTickHandle);
	PostTickHandle.Reset();
}

UObject* SSubobjectEditor::GetObjectContext() const
{
	return ObjectContext.Get(nullptr);
}

FSubobjectDataHandle SSubobjectEditor::GetObjectContextHandle() const
{
	if (RootNodes.Num() > 0)
	{
		if (RootNodes[0]->GetObject() == GetObjectContext())
		{
			return RootNodes[0]->GetDataHandle();
		}
	}

	return FSubobjectDataHandle::InvalidHandle;
	
}

FSubobjectData* SSubobjectEditor::GetDataContextHandle() const
{
	if (RootNodes.Num() > 0)
	{
		return RootNodes[0]->GetDataHandle().GetData();
	}
	return nullptr;
}

UBlueprint* SSubobjectEditor::GetBlueprint() const
{
	if (AActor* Actor = Cast<AActor>(GetObjectContext()))
	{
		const UClass* ActorClass = Actor->GetClass();
		check(ActorClass != nullptr);

		return UBlueprint::GetBlueprintFromClass(ActorClass);
	}

	return nullptr;
}

void SSubobjectEditor::DepthFirstTraversal(const FSubobjectEditorTreeNodePtrType& InNodePtr, TSet<FSubobjectEditorTreeNodePtrType>& OutVisitedNodes, const TFunctionRef<void(const FSubobjectEditorTreeNodePtrType&)> InFunction) const
{
	if (InNodePtr.IsValid()
		&& ensureMsgf(!OutVisitedNodes.Contains(InNodePtr), TEXT("Already visited node: %s (Parent: %s)"),
		              *InNodePtr->GetDisplayString(),
		              InNodePtr->GetParent().IsValid() ? *InNodePtr->GetParent()->GetDisplayString() : TEXT("NULL")))
	{
		InFunction(InNodePtr);
		OutVisitedNodes.Add(InNodePtr);

		for (const FSubobjectEditorTreeNodePtrType& Child : InNodePtr->GetChildren())
		{
			DepthFirstTraversal(Child, OutVisitedNodes, InFunction);
		}
	}
}

void SSubobjectEditor::GetCollapsedNodes(const FSubobjectEditorTreeNodePtrType& InNodePtr, TSet<FSubobjectEditorTreeNodePtrType>& OutCollapsedNodes) const
{
	TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
	DepthFirstTraversal(InNodePtr, VisitedNodes,
        [TreeWidget = this->TreeWidget, &OutCollapsedNodes](
        const FSubobjectEditorTreeNodePtrType& InNodePtr)
        {
            if (InNodePtr->GetChildren().Num() > 0 && !TreeWidget->IsItemExpanded(InNodePtr))
            {
                OutCollapsedNodes.Add(InNodePtr);
            }
        });
}

FSubobjectEditorTreeNodePtrType SSubobjectEditor::FindSlateNodeForHandle(const FSubobjectDataHandle& InHandleToFind, FSubobjectEditorTreeNodePtrType InStartNodePtr) const
{
	FSubobjectEditorTreeNodePtrType OutNodePtr;
	if (InHandleToFind.IsValid() && RootNodes.Num() > 0)
	{
		const FSubobjectDataSubsystemContextData_SingleUObjectContextObject* const AsSingleUObject = InHandleToFind.GetData()->GetContextData().GetPtr<FSubobjectDataSubsystemContextData_SingleUObjectContextObject>();
		const FSubobjectDataSubsystemContextData_TedsRow* const AsTedsRow = InHandleToFind.GetData()->GetContextData().GetPtr<FSubobjectDataSubsystemContextData_TedsRow>();

		if (AsSingleUObject)
		{
			FSubobjectData* DataToFind = InHandleToFind.GetSharedDataPtr().Get();
			TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
			DepthFirstTraversal(RootNodes[0], VisitedNodes,
				[&OutNodePtr, &DataToFind](
					const FSubobjectEditorTreeNodePtrType& CurNodePtr)
				{
					if (CurNodePtr->GetDataHandle().IsValid())
					{
						if (CurNodePtr->GetDataHandle().GetSharedDataPtr()->GetObject() == DataToFind->GetObject())
						{
							OutNodePtr = CurNodePtr;
						}
					}
				});
		}
		else if (AsTedsRow)
		{
			using namespace UE::Editor::DataStorage;
			const ICoreProvider* const CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			check(CoreProvider);

			auto GetObjectFromTedsRow = [CoreProvider](RowHandle Row) -> UObject*
			{
				if (const FSubobjectDataSubsystemRowReferenceCollection* const Collection = CoreProvider->GetColumn<FSubobjectDataSubsystemRowReferenceCollection>(Row))
				{
					for (const RowHandle CollectionRow : Collection->Rows)
					{
						const FTypedElementUObjectColumn* const UObjectColumn = CoreProvider->GetColumn<FTypedElementUObjectColumn>(CollectionRow);
						if (UObjectColumn)
						{
							return UObjectColumn->Object.Get();
						}
					}
				}
				return nullptr;
			};

			if (const UObject* const QueryObject = GetObjectFromTedsRow(AsTedsRow->GetRow()))
			{
				TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
				DepthFirstTraversal(RootNodes[0], VisitedNodes, [&OutNodePtr, QueryObject, GetObjectFromTedsRow, CoreProvider](const FSubobjectEditorTreeNodePtrType& CurNodePtr)
				{
					if (CurNodePtr->GetDataHandle().IsValid())
					{
						if (const FSubobjectDataSubsystemContextData_TedsRow* const TestTedsRow = CurNodePtr->GetDataHandle().GetData()->GetContextData().GetPtr<FSubobjectDataSubsystemContextData_TedsRow>())
						{
							const UObject* const TestObject = GetObjectFromTedsRow(TestTedsRow->GetRow());
							if (QueryObject == TestObject)
							{
								OutNodePtr = CurNodePtr;
							}
						}
					}
				});
			}
		}

	}

	return OutNodePtr;
}

FSubobjectEditorTreeNodePtrType SSubobjectEditor::FindSlateNodeForVariableName(FName InVariableName) const
{
	FSubobjectEditorTreeNodePtrType OutNodePtr;
	if (RootNodes.Num() > 0)
	{
		TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
		DepthFirstTraversal(RootNodes[0], VisitedNodes,
			[&OutNodePtr, InVariableName](
			const FSubobjectEditorTreeNodePtrType& CurNodePtr)
			{
				if (CurNodePtr->GetDataHandle().IsValid())
				{
					if (CurNodePtr->GetDataHandle().GetSharedDataPtr()->GetVariableName() == InVariableName)
					{
						OutNodePtr = CurNodePtr;
					}
				}
			});
	}

	return OutNodePtr;
}

FSubobjectEditorTreeNodePtrType SSubobjectEditor::FindSlateNodeForObject(const UObject* InObject, bool bIncludeAttachmentComponents) const
{
	FSubobjectEditorTreeNodePtrType OutNodePtr;

	if (InObject && RootNodes.Num() > 0)
	{
		TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
		UBlueprint* BP = GetBlueprint();

		DepthFirstTraversal(RootNodes[0], VisitedNodes,
            [&OutNodePtr, InObject, BP](
            const FSubobjectEditorTreeNodePtrType& CurNodePtr)
            {
                if(CurNodePtr->GetDataHandle().IsValid())
                {
					if (const FSubobjectData* Data = CurNodePtr->GetDataHandle().GetData())
					{
						if (Data->GetObject() == InObject)
						{
							OutNodePtr = CurNodePtr;
						}
						// Handle BP inherited subobjects that are on instances
						else if (!OutNodePtr && Data->GetObjectForBlueprint(BP) == InObject->GetArchetype())
						{
							OutNodePtr = CurNodePtr;
						}
					}
                }
            });

		// If we didn't find it in the tree, step up the chain to the parent of the given component and recursively see if that is in the tree (unless the flag is false)
		if (!OutNodePtr.IsValid() && bIncludeAttachmentComponents)
		{
			const USceneComponent* SceneComponent = Cast<const USceneComponent>(InObject);
			if (SceneComponent && SceneComponent->GetAttachParent())
			{
				return FindSlateNodeForObject(SceneComponent->GetAttachParent(), bIncludeAttachmentComponents);
			}
		}
	}

	return OutNodePtr;
}

void SSubobjectEditor::SetNodeExpansionState(FSubobjectEditorTreeNodePtrType InNodeToChange, const bool bIsExpanded)
{
	if (TreeWidget.IsValid() && InNodeToChange.IsValid())
	{
		TreeWidget->SetItemExpansion(InNodeToChange, bIsExpanded);
	}
}

void SSubobjectEditor::SetItemExpansionRecursive(FSubobjectEditorTreeNodePtrType Model, bool bInExpansionState)
{
	SetNodeExpansionState(Model, bInExpansionState);
	for (const FSubobjectEditorTreeNodePtrType& Child : Model->GetChildren())
	{
		if (Child.IsValid())
		{
			SetItemExpansionRecursive(Child, bInExpansionState);
		}
	}
}

void SSubobjectEditor::DumpTree()
{
	/* Example:

		[ACTOR] MyBlueprint (self)
		|
		[SEPARATOR]
		|
		DefaultSceneRoot (Inherited)
		|
		+- StaticMesh (Inherited)
		|  |
		|  +- Scene4 (Inherited)
		|  |
		|  +- Scene (Inherited)
		|     |
		|     +- Scene1 (Inherited)
		|
		+- Scene2 (Inherited)
		|  |
		|  +- Scene3 (Inherited)
		|
		[SEPARATOR]
		|
		ProjectileMovement (Inherited)
	*/

	UE_LOGF(LogSubobjectEditor, Log, "---------------------");
	UE_LOGF(LogSubobjectEditor, Log, " STreeView NODE DUMP");
	UE_LOGF(LogSubobjectEditor, Log, "---------------------");

	//TSet<FSubobjectEditorTreeNodePtrType> VisitedNodes;
	//DepthFirstTraversal(RootNodes[0], VisitedNodes, [](const FSubobjectEditorTreeNodePtrType& InNode)
	//{
	//	UE_LOGF(LogSubobjectEditor, Log, "");
	//});
}

bool SSubobjectEditor::IsEditingAllowed() const
{
	return AllowEditing.Get() && nullptr == GEditor->PlayWorld;
}

TSharedPtr<SWidget> SSubobjectEditor::GetToolButtonsBox() const
{
	return ButtonBox;
}

void SSubobjectEditor::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(SubobjectTree_ContextMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(SubobjectTree_ContextMenuName);
		Menu->AddDynamicSection("SCSEditorDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			USubobjectEditorMenuContext* ContextObject = InMenu->FindContext<USubobjectEditorMenuContext>();
			if (ContextObject && ContextObject->SubobjectEditor.IsValid())
			{
				ContextObject->SubobjectEditor.Pin()->PopulateContextMenu(InMenu);
			}
		}));
	}
}

TSharedPtr<SWidget> SSubobjectEditor::CreateContextMenu()
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedItems = TreeWidget->GetSelectedItems();

	if (SelectedItems.Num() > 0 || CanPasteNodes())
	{
		RegisterContextMenu();
		USubobjectEditorMenuContext* ContextObject = NewObject<USubobjectEditorMenuContext>();
		ContextObject->SubobjectEditor = SharedThis(this);
		FToolMenuContext ToolMenuContext(CommandList, TSharedPtr<FExtender>(), ContextObject);
		return UToolMenus::Get()->GenerateWidget(SubobjectTree_ContextMenuName, ToolMenuContext);
	}
	return TSharedPtr<SWidget>();
}

void SSubobjectEditor::PopulateContextMenu(UToolMenu* Menu)
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedItems = TreeWidget->GetSelectedItems();

	if(SelectedItems.Num() > 0 || CanPasteNodes())
	{
		bool bOnlyShowPasteOption = false;

		if(SelectedItems.Num() > 0)
		{
			FSubobjectData* SelectedData = SelectedItems[0]->GetDataSource();
			const AActor* SelectedActor =  SelectedData->GetObject<AActor>();
			if (SelectedItems.Num() == 1 && SelectedActor)
			{
				if (SelectedData->IsChildActor())
				{
					// Include specific context menu options for a single child actor node selection
					SubobjectEditorUtils::FillChildActorContextMenuOptions(Menu, SelectedItems[0]);
				}
				else
				{
					bOnlyShowPasteOption = true;
				}
			}
			else
			{
				for (const FSubobjectEditorTreeNodePtrType& SelectedNode : SelectedItems)
				{
					if (!SelectedNode->IsComponentNode())
					{
						bOnlyShowPasteOption = true;
						break;
					}
				}

				if (!bOnlyShowPasteOption)
				{
					bool bIsChildActorSubtreeNodeSelected = false;
					
					TArray<UActorComponent*> SelectedComponents;
					for(const FSubobjectEditorTreeNodePtrType& SelectedNodePtr : SelectedItems)
					{
						check(SelectedNodePtr->IsValid());
						// Get the component template associated with the selected node
						const UActorComponent* ComponentTemplate = SelectedNodePtr->GetComponentTemplate();
						if (ComponentTemplate)
						{
							// #TODO_BH Remove this const cast
							SelectedComponents.Add(const_cast<UActorComponent*>(ComponentTemplate));
						}

						// Determine if any selected node belongs to a child actor template
						if (!bIsChildActorSubtreeNodeSelected)
						{
							bIsChildActorSubtreeNodeSelected = SelectedNodePtr->IsChildSubtreeNode();
						}
					}

					PopulateContextMenuImpl(Menu, SelectedItems, bIsChildActorSubtreeNodeSelected);
				}
			}
		}
		else
		{
			bOnlyShowPasteOption = true;
		}

		if (bOnlyShowPasteOption)
		{
			FToolMenuSection& Section = Menu->AddSection("PasteComponent", LOCTEXT("EditComponentHeading", "Edit"));
			{
				Section.AddMenuEntry(FGenericCommands::Get().Paste);
			}
		}
	}
}

void SSubobjectEditor::GetSelectedItemsForContextMenu(TArray<FComponentEventConstructionData>& OutSelectedItems) const
{
	TArray<FSubobjectEditorTreeNodePtrType> SelectedTreeItems = TreeWidget->GetSelectedItems();
	for (auto NodeIter = SelectedTreeItems.CreateConstIterator(); NodeIter; ++NodeIter)
	{
		FComponentEventConstructionData NewItem;
		const FSubobjectEditorTreeNodePtrType& TreeNode = *NodeIter;
		const FSubobjectData* Data = TreeNode->GetDataSource();
		NewItem.VariableName = Data->GetVariableName();
		const UObject* ContextObject = GetObjectContext();
		const AActor* ContextAsActor = Cast<AActor>(ContextObject);
		if (ContextObject && !ContextObject->HasAnyFlags(RF_ClassDefaultObject) && ContextAsActor)
		{
			NewItem.Component = const_cast<UActorComponent*>(Data->FindComponentInstanceInActor(ContextAsActor));
		}
		else if (const UObject* BlueprintObject = Data->GetObjectForBlueprint(GetBlueprint()))
		{
			NewItem.Component = const_cast<UObject*>(BlueprintObject);
		}
		else if (Data->GetContextData().IsValid())
		{
			const TInstancedStruct<FSubobjectDataSubsystemContextDataBase>& ContextData = Data->GetContextData();
			const auto* AsTedsRow = ContextData.GetPtr<FSubobjectDataSubsystemContextData_TedsRow>();
			const auto* AsSingleUObject = ContextData.GetPtr<FSubobjectDataSubsystemContextData_SingleUObjectContextObject>();

			if (AsTedsRow)
			{
				using namespace UE::Editor::DataStorage;
				ICoreProvider* CoreProvider = UE::Editor::DataStorage::GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				check(CoreProvider);

				RowHandle ComponentRowHandle = AsTedsRow->GetRow();
				if (const FSubobjectDataSubsystemRowReferenceCollection* Collection = CoreProvider->GetColumn<FSubobjectDataSubsystemRowReferenceCollection>(ComponentRowHandle))
				{
					for (UE::Editor::DataStorage::RowHandle Row : Collection->Rows)
					{
						const FTypedElementUObjectColumn* UObjectColumn = CoreProvider->GetColumn<FTypedElementUObjectColumn>(Row);
						if (UObjectColumn)
						{
							UObject* Object = UObjectColumn->Object.Get();
							FComponentEventConstructionData& ContextItem = OutSelectedItems.AddDefaulted_GetRef();
							ContextItem.Component = Object;
						}
					}
				}
				
			}
			else if (AsSingleUObject)
			{
				if (TStrongObjectPtr<UObject> Object = AsSingleUObject->Object.Pin())
				{
					FComponentEventConstructionData& ContextItem = OutSelectedItems.AddDefaulted_GetRef();
					ContextItem.Component = Object.Get();
				}
			}

			// Skip adding the partially constructed NewItem since we handled that in our loop above
			continue;
		}
		OutSelectedItems.Add(NewItem);
	}
}

TSubclassOf<UActorComponent> SSubobjectEditor::GetComponentTypeFilterToApply() const
{
	UObject* ObjectContextPtr = GetObjectContext();
	TSubclassOf<UActorComponent> ComponentType = UICustomization.IsValid()
		                                             ? UICustomization->GetComponentTypeFilter(MakeArrayView(&ObjectContextPtr, 1))
		                                             : nullptr;
	if (!ComponentType)
	{
		ComponentType = ComponentTypeFilter.Get();
	}
	return ComponentType;
}

bool SSubobjectEditor::RefreshFilteredState(FSubobjectEditorTreeNodePtrType TreeNode, bool bRecursive)
{
	const UClass* FilterType = GetComponentTypeFilterToApply();

	FString FilterText = FText::TrimPrecedingAndTrailing(GetFilterText()).ToString();
	TArray<FString> FilterTerms;
	FilterText.ParseIntoArray(FilterTerms, TEXT(" "), /*CullEmpty =*/true);

	TreeNode->RefreshFilteredState(FilterType, FilterTerms, bRecursive);
	return TreeNode->IsFlaggedForFiltration();
}

FText SSubobjectEditor::GetFilterText() const
{
	return FilterBox->GetText();
}

void SSubobjectEditor::PromoteToBlueprint() const
{
	if (AActor* ActorContext = Cast<AActor>(GetObjectContext()))
	{
		FCreateBlueprintFromActorDialog::OpenDialog(ECreateBlueprintFromActorMode::Subclass, ActorContext);
	}
}

FReply SSubobjectEditor::OnPromoteToBlueprintClicked()
{
	PromoteToBlueprint();
	return FReply::Handled();
}

void SSubobjectEditor::OnOpenBlueprintEditor(bool bForceCodeEditing) const
{
	if (AActor* ActorInstance = Cast<AActor>(GetObjectContext()))
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(ActorInstance->GetClass()->ClassGeneratedBy))
		{
			if (bForceCodeEditing && (Blueprint->UbergraphPages.Num() > 0))
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Blueprint->GetLastEditedUberGraph());
			}
			else
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
			}
		}
	}
}

////////////////////////////////////////////////
// Class filters for creating new components
class FComponentClassParentFilter : public IClassViewerFilter
{
public:
	FComponentClassParentFilter(const TSubclassOf<UActorComponent>& InComponentClass) : ComponentClass(InComponentClass)
	{
	}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return InClass->IsChildOf(ComponentClass);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return InUnloadedClassData->IsChildOf(ComponentClass);
	}

	TSubclassOf<UActorComponent> ComponentClass;
};

typedef FComponentClassParentFilter FNativeComponentClassParentFilter;

class FBlueprintComponentClassParentFilter : public FComponentClassParentFilter
{
public:
	FBlueprintComponentClassParentFilter(const TSubclassOf<UActorComponent>& InComponentClass)
		: FComponentClassParentFilter(InComponentClass)
	{}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return
			FComponentClassParentFilter::IsClassAllowed(InInitOptions, InClass, InFilterFuncs) &&
			FKismetEditorUtilities::CanCreateBlueprintOfClass(InClass);
	}
};

UClass* SSubobjectEditor::SpawnCreateNewCppComponentWindow(TSubclassOf<UActorComponent> ComponentClass)
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));

	FString AddedClassName;
	auto OnCodeAddedToProject = [&AddedClassName](const FString& ClassName, const FString& ClassPath,
	                                              const FString& ModuleName)
	{
		if (!ClassName.IsEmpty() && !ClassPath.IsEmpty())
		{
			AddedClassName = FString::Printf(TEXT("/Script/%s.%s"), *ModuleName, *ClassName);
		}
	};

	FGameProjectGenerationModule::Get().OpenAddCodeToProjectDialog(
		FAddToProjectConfig()
		.WindowTitle(LOCTEXT("AddNewC++Component", "Add C++ Component"))
		.ParentWindow(ParentWindow)
		.Modal()
		.OnAddedToProject(FOnAddedToProject::CreateLambda(OnCodeAddedToProject))
		.FeatureComponentClasses()
		.AllowableParents(MakeShareable(new FNativeComponentClassParentFilter(ComponentClass)))
		.DefaultClassPrefix(TEXT("New"))
	);


	return LoadClass<UActorComponent>(nullptr, *AddedClassName, nullptr, LOAD_None, nullptr);
}

UClass* SSubobjectEditor::SpawnCreateNewBPComponentWindow(TSubclassOf<UActorComponent> ComponentClass)
{
	UClass* NewClass = nullptr;

	auto OnAddedToProject = [&](const FString& ClassName, const FString& PackagePath, const FString& ModuleName)
	{
		if (!ClassName.IsEmpty() && !PackagePath.IsEmpty())
		{
			if (UPackage* Package = FindPackage(nullptr, *PackagePath))
			{
				if (UBlueprint* NewBP = FindObjectFast<UBlueprint>(Package, *ClassName))
				{
					NewClass = NewBP->GeneratedClass;

					TArray<UObject*> Objects;
					Objects.Emplace(NewBP);
					GEditor->SyncBrowserToObjects(Objects);

					// Open the editor for the new blueprint
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);
				}
			}
		}
	};

	FGameProjectGenerationModule::Get().OpenAddBlueprintToProjectDialog(
		FAddToProjectConfig()
		.WindowTitle(LOCTEXT("AddNewBlueprintComponent", "Add Blueprint Component"))
		.ParentWindow(FSlateApplication::Get().FindWidgetWindow(SharedThis(this)))
		.Modal()
		.AllowableParents(MakeShareable(new FBlueprintComponentClassParentFilter(ComponentClass)))
		.FeatureComponentClasses()
		.OnAddedToProject(FOnAddedToProject::CreateLambda(OnAddedToProject))
		.DefaultClassPrefix(TEXT("New"))
	);

	return NewClass;
}

FSubobjectDataHandle SSubobjectEditor::PerformComboAddClass(TSubclassOf<UActorComponent> ComponentClass, EComponentCreateAction::Type ComponentCreateAction, UObject* AssetOverride)
{
	FSubobjectDataHandle NewComponentHandle = FSubobjectDataHandle::InvalidHandle;

	// Display a class picked for which class to use for the new component
	UClass* NewClass = ComponentClass;

	// Display a class creation menu if the user wants to create a new class
	if (ComponentCreateAction == EComponentCreateAction::CreateNewCPPClass)
	{
		NewClass = SpawnCreateNewCppComponentWindow(ComponentClass);
	}
	else if (ComponentCreateAction == EComponentCreateAction::CreateNewBlueprintClass)
	{
		NewClass = SpawnCreateNewBPComponentWindow(ComponentClass);
	}

	USubobjectDataSubsystem* SubobjectSystem = USubobjectDataSubsystem::Get();

	if (NewClass && SubobjectSystem)
	{
		TUniquePtr<FScopedTransaction> AddTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("AddComponent", "Add Component"));

		FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();
		USelection* Selection = GEditor->GetSelectedObjects();

		bool bAddedComponent = false;
		FText OutFailReason;

		// This adds components according to the type selected in the drop down. If the user
		// has the appropriate objects selected in the content browser then those are added,
		// else we go down the previous route of adding components by type.
		
		// Furthermore don't try to match up assets for USceneComponent it will match lots of things and doesn't have any nice behavior for asset adds 
		if (Selection->Num() > 0 && !AssetOverride && NewClass != USceneComponent::StaticClass())
		{
			for (FSelectionIterator ObjectIter(*Selection); ObjectIter; ++ObjectIter)
			{
				UObject* Object = *ObjectIter;
				UClass* Class = Object->GetClass();

				TArray<TSubclassOf<UActorComponent>> ComponentClasses = FComponentAssetBrokerage::GetComponentsForAsset(Object);

				// if the selected asset supports the selected component type then go ahead and add it
				for (int32 ComponentIndex = 0; ComponentIndex < ComponentClasses.Num(); ComponentIndex++)
				{
					if (ComponentClasses[ComponentIndex]->IsChildOf(NewClass))
					{
						const TArray<FSubobjectDataHandle>& SelectedHandles = GetSelectedHandles();
						const FSubobjectDataHandle& ParentHandle = SelectedHandles.Num() > 0 ? SelectedHandles[0] : CachedRootHandle;
						ensureMsgf(ParentHandle.IsValid(), TEXT("Attempting to add a component from an invalid selection!"));
						
						NewComponentHandle = AddNewSubobject(ParentHandle, NewClass, Object, OutFailReason, MoveTemp(AddTransaction));
						bAddedComponent = true;
						break;
					}
				}
			}
		}

		if (!bAddedComponent)
		{
			// Attach this component to the override asset first, but if none is given then use the actor context			
			FSubobjectDataHandle ParentHandle = SubobjectSystem->FindHandleForObject(CachedRootHandle, AssetOverride);
			
			if(!ParentHandle.IsValid())
			{
				// If we have something selected, then we should attach it to that
				TArray<FSubobjectEditorTreeNodePtrType> SelectedTreeNodes = TreeWidget->GetSelectedItems();
				if(SelectedTreeNodes.Num() > 0)
				{
					ParentHandle = SelectedTreeNodes[0]->GetDataHandle();
				}
				// Otherwise fall back to the root node
				else
				{
					ParentHandle = RootNodes.Num() > 0
						               ? RootNodes[0]->GetDataHandle()
						               : FSubobjectDataHandle::InvalidHandle;					
				}
			}
			
			if(ParentHandle.IsValid())
			{
				NewComponentHandle = AddNewSubobject(ParentHandle, NewClass, AssetOverride, OutFailReason, MoveTemp(AddTransaction));
			}
		}

		// We failed adding a new component, display why with a slate notif
		if(!NewComponentHandle.IsValid())
		{
			if(OutFailReason.IsEmpty())
			{
				OutFailReason = LOCTEXT("AddComponentFailed_Generic", "Failed to add component!");
			}
			
			FNotificationInfo Info(OutFailReason);
			Info.Image = FAppStyle::GetBrush(TEXT("Icons.Error"));
			Info.bFireAndForget = true;
			Info.bUseSuccessFailIcons = false;
			Info.ExpireDuration = 5.0f;

			FSlateNotificationManager::Get().AddNotification(Info);
		}
		else
		{
			UpdateTree();
			
			// Set focus to the newly created subobject
			FSubobjectEditorTreeNodePtrType NewNode = FindSlateNodeForHandle(NewComponentHandle);
			if(NewNode != nullptr)
			{
				TreeWidget->SetSelection(NewNode);
				OnRenameComponent(MoveTemp(AddTransaction));
			}
		}
	}

	return NewComponentHandle;
}

EVisibility SSubobjectEditor::GetPromoteToBlueprintButtonVisibility() const
{
	UObject* ObjectContextPtr = GetObjectContext();
	return (UICustomization.IsValid() && UICustomization->HideBlueprintButtons(MakeArrayView(&ObjectContextPtr, 1)))
	       || (ShowInlineSearchWithButtons())
	       || (GetBlueprint() != nullptr)
		       ? EVisibility::Collapsed
		       : EVisibility::Visible;
}

EVisibility SSubobjectEditor::GetEditBlueprintButtonVisibility() const
{
	UObject* ObjectContextPtr = GetObjectContext();
	return (UICustomization.IsValid() && UICustomization->HideBlueprintButtons(MakeArrayView(&ObjectContextPtr, 1)))
	       || (ShowInlineSearchWithButtons())
	       || (GetBlueprint() == nullptr)
		       ? EVisibility::Collapsed
		       : EVisibility::Visible;
}

EVisibility SSubobjectEditor::GetComponentClassComboButtonVisibility() const
{
	UObject* ObjectContextPtr = GetObjectContext();
	return (HideComponentClassCombo.Get() 
	|| (UICustomization.IsValid() && UICustomization->HideAddComponentButton(MakeArrayView(&ObjectContextPtr, 1))))
	? EVisibility::Collapsed : EVisibility::Visible;
}

void SSubobjectEditor::Utils::PopulateHandlesArray(const TArray<FSubobjectEditorTreeNodePtrType>& SlateNodePtrs, TArray<FSubobjectDataHandle>& OutHandles)
{
	for(const FSubobjectEditorTreeNodePtrType& DroppedNodePtr : SlateNodePtrs)
    {
    	OutHandles.Add(DroppedNodePtr->GetDataHandle());
    }
}

void SSubobjectEditor::ForEachNode(TFunctionRef<void(FSubobjectEditorTreeNode& Data)> Ftor)
{
	for(const TSharedPtr<class FSubobjectEditorTreeNode>& Node : RootNodes)
	{
		Node->ForEachNodeImpl(Ftor);
	}
}

void SSubobjectEditor::RefreshComponentTypesList()
{
	if (ComponentClassCombo.IsValid())
	{
		ComponentClassCombo->UpdateComponentClassList();
	}
}

TArray<UObject*> USubobjectEditorMenuContext::GetSelectedObjects() const
{
	TArray<UObject*> Result;
	if (TSharedPtr<SSubobjectEditor> Pinned = SubobjectEditor.Pin())
	{
		TArray<FComponentEventConstructionData> SelectedItems;
		Pinned->GetSelectedItemsForContextMenu(SelectedItems);
		for (FComponentEventConstructionData& SelectedItemConstructionData : SelectedItems)
		{
			if (UObject* SelectedObject = SelectedItemConstructionData.Component.Get())
			{
				Result.Add(SelectedObject);
			}
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE