// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSubobjectEditorModule.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FSubobjectEditorModule, SubobjectEditor)