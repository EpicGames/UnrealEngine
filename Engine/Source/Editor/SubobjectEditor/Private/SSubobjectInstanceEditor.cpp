// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSubobjectInstanceEditor.h"

#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorClassUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GraphEditorActions.h"
#include "IDocumentation.h"
#include "ISCSEditorUICustomization.h"	// #TODO_BH Rename this to subobject
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"		// ApplyInstanceChangesToBlueprint
#include "SComponentClassCombo.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "Styling/SlateIconFinder.h"
#include "SubobjectEditorExtensionContext.h"
#include "Subsystems/PanelExtensionSubsystem.h"	// SExtensionPanel
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SSubobjectInstanceEditor"

extern UNREALED_API UUnrealEdEngine* GUnrealEd;

////////////////////////////////////////////////
// SSubobjectInstanceEditor

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSubobjectInstanceEditor::Construct(const FArguments& InArgs)
{
	ObjectContext = InArgs._ObjectContext;
	OnSelectionUpdated = InArgs._OnSelectionUpdated;
	OnItemDoubleClicked = InArgs._OnItemDoubleClicked;
	AllowEditing = InArgs._AllowEditing;
	bAllowTreeUpdates = true;
	UICustomization = InArgs._SCSEditorUICustomization;

	InitScopeRow(TEXT("Instance Subobject Editor"));

	CreateCommandList();

	// Build the tree widget
	FSlateBrush const* MobilityHeaderBrush = FAppStyle::GetBrush(TEXT("ClassIcon.ComponentMobilityHeaderIcon"));
	
	ConstructTreeWidget();

	// Should only be true when used in the blueprints details panel
	const bool bInlineSearchBarWithButtons = ShowInlineSearchWithButtons();

	TSharedPtr<SWidget> Contents;

	TSharedPtr<SVerticalBox> HeaderBox;
	TSharedPtr<SWidget> SearchBar = SAssignNew(FilterBox, SSearchBox)
		.HintText(!bInlineSearchBarWithButtons ? LOCTEXT("SearchComponentsHint", "Search Components") : LOCTEXT("SearchHint", "Search"))
		.OnTextChanged(this, &SSubobjectInstanceEditor::OnFilterTextChanged)
		.Visibility(this, &SSubobjectInstanceEditor::GetComponentsFilterBoxVisibility);

	FMenuBuilder EditBlueprintMenuBuilder = CreateMenuBuilder();

	// Extension context for new boi
	USubobjectEditorExtensionContext* ExtensionContext = NewObject<USubobjectEditorExtensionContext>();
	ExtensionContext->SubobjectEditor = SharedThis(this);
	ExtensionContext->AddToRoot();

	ButtonBox = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.Padding(0.0f, 0.0f, 4.0f, 0.0f)
	.AutoWidth()
	[
		SNew(SComponentClassCombo)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.AddComponent")))
		.Visibility(this, &SSubobjectInstanceEditor::GetComponentClassComboButtonVisibility)
		.OnSubobjectClassSelected(this, &SSubobjectInstanceEditor::PerformComboAddClass)
		.ToolTipText(LOCTEXT("AddComponent_Tooltip", "Adds a new component to this actor"))
		.IsEnabled(true)
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SAssignNew(ExtensionPanel, SExtensionPanel)
		.ExtensionPanelID("SCSEditor.NextToAddComponentButton")
		.ExtensionContext(ExtensionContext)
	]
	
	// horizontal slot index #2 => reserved for BP-editor search bar (see 'ButtonBox' and 'SearchBarHorizontalSlotIndex' usage below)

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "DetailsView.NameAreaButton")
		.ContentPadding(0)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.ConvertToBlueprint")))
		.Visibility(this, &SSubobjectInstanceEditor::GetPromoteToBlueprintButtonVisibility)
		.OnClicked(this, &SSubobjectInstanceEditor::OnPromoteToBlueprintClicked)
		.ToolTip(IDocumentation::Get()->CreateToolTip(
			LOCTEXT("PromoteToBluerprintTooltip", "Converts this actor into a reusable Blueprint Class that can have script behavior" ),
			nullptr,
			TEXT("Shared/LevelEditor"),
			TEXT("ConvertToBlueprint")))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Blueprints"))
		]
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "DetailsView.NameAreaComboButton")
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ContentPadding(FMargin(0, 2, 0, 1))
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Actor.EditBlueprint")))
		.Visibility(this, &SSubobjectInstanceEditor::GetEditBlueprintButtonVisibility)
		.ToolTipText(LOCTEXT("EditActorBlueprint_Tooltip", "Edit the Blueprint for this Actor"))
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Blueprints"))
		]
		.MenuContent()
		[
			EditBlueprintMenuBuilder.MakeWidget()
		]
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.Padding(4, 0, 0, 0)
	.AutoWidth()
	[
		FEditorClassUtils::GetDynamicDocumentationLinkWidget(TAttribute<const UClass*>::CreateLambda([this]()
			{
				UObject* Obj = GetObjectContext();
				return Obj != nullptr ? Obj->GetClass() : nullptr;
			}))
	];

	Contents = SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.VAlign(VAlign_Top)
	.Padding(4.f, 0, 4.f, 4.f)
	[
		SAssignNew(HeaderBox, SVerticalBox)
	]

	+ SVerticalBox::Slot()
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("SCSEditor.Background"))
			.Padding(4.f)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ComponentsPanel")))
			.Visibility(this, &SSubobjectInstanceEditor::GetComponentsTreeVisibility)
			[
				TreeWidget.ToSharedRef()
			]
		]
		+ SOverlay::Slot()
		[
			SAssignNew(TreeViewOverlay, SBox)
			.Visibility_Lambda([this]()
			{
				const bool bTreeIsVisible = GetComponentsTreeVisibility() == EVisibility::Visible;
				const bool bSupportsOverlay = UICustomization && UICustomization->SupportsTreeViewOverlay(*this);
				
				if (bTreeIsVisible && bSupportsOverlay)
				{
					// TODO: Could add more criteria like whether the thing we're dragging can actually be dropped here
					const bool bIsDragging = FSlateApplication::Get().IsDragDropping();
					return bIsDragging ? EVisibility::Visible : EVisibility::HitTestInvisible;
				} 

				return EVisibility::Collapsed;
			})
			.Content()
			[
				UICustomization ? UICustomization->ConstructTreeViewOverlay(SharedThis(this)) : SNullWidget::NullWidget
			]
			
		]
	];

	// Only insert the buttons and search bar in the Blueprints version
	if (bInlineSearchBarWithButtons)
	{
		ButtonBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 3.0f)
		[
			SearchBar.ToSharedRef()
		];

		HeaderBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoHeight()
		[
			ButtonBox.ToSharedRef()
		];
	}

	this->ChildSlot
	[
		Contents.ToSharedRef()
	];

	// Populate the tree with subobject data
	UpdateTree();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SSubobjectInstanceEditor::UpdateTree(bool bRegenerateTreeNodes)
{
	SSubobjectEditor::UpdateTree(bRegenerateTreeNodes);

	if (TreeViewOverlay.IsValid())
	{
		if (UICustomization && UICustomization->SupportsTreeViewOverlay(*this))
		{
			TreeViewOverlay->SetContent(UICustomization->ConstructTreeViewOverlay(SharedThis(this)));
		}
	}
}


void SSubobjectInstanceEditor::OnDeleteNodes()
{
	const FScopedTransaction Transaction( LOCTEXT("RemoveComponents", "Remove Components") );

	// Invalidate any active component in the visualizer
	GUnrealEd->ComponentVisManager.ClearActiveComponentVis();

	// Gather the handles of the components that we want to delete
	TArray<FSubobjectDataHandle> HandlesToDelete;
	TArray<FSubobjectEditorTreeNodePtrType> SelectedNodes = GetSelectedNodes();
	for(const FSubobjectEditorTreeNodePtrType& Node : SelectedNodes)
	{
		check(Node->IsValid());
		const FSubobjectData* Data = Node->GetDataSource();
		if(Data && Data->IsComponent())
		{
			HandlesToDelete.Add(Data->GetHandle());
		}
	}

	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		FSubobjectDataHandle HandleToSelect;
		int32 NumDeleted = System->DeleteSubobjects(RootNodes[0]->GetDataHandle(), HandlesToDelete, HandleToSelect);

		if(NumDeleted > 0)
		{			
			FSubobjectEditorTreeNodePtrType NodeToSelect = HandleToSelect.IsValid() ? FindSlateNodeForHandle(HandleToSelect) : GetSceneRootNode();
			if(NodeToSelect.IsValid())
			{
				TreeWidget->SetSelection(NodeToSelect);
			}
			// If there are no components left, then fall back to the generic root node.
			// This may be the case if you have deleted all components on a native instance actor
			else if(!RootNodes.IsEmpty())
			{
				TreeWidget->SetSelection(RootNodes[0]);
			}
			
			UpdateTree();

			// Do this AFTER marking the Blueprint as modified
			UpdateSelectionFromNodes(TreeWidget->GetSelectedItems());
		}
	}
}

void SSubobjectInstanceEditor::CopySelectedNodes()
{
	TArray<FSubobjectDataHandle> SelectedHandles = GetSelectedHandles();
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		return System->CopySubobjects(SelectedHandles, /* BpContext = */ nullptr);
	}
}

void SSubobjectInstanceEditor::OnDuplicateComponent()
{
	TArray<FSubobjectDataHandle> SelectedNodes = GetSelectedHandles();
	if(SelectedNodes.Num() <= 0)
	{
		return;
	}

	USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
	if (!System)
	{
		return;
	}

	bool ClearedFocus = false;

	// Check if the focus is currently in some sort of textbox
	if (const TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetKeyboardFocusedWidget())
	{
		if (ShouldClearFocusOnDuplicate(FocusedWidget))
		{
			// Force the text box being edited (if any) to commit its text. The duplicate operation may trigger a regeneration of the tree view,
			// releasing all row widgets. If one row was in edit mode (rename/rename on create), it was released before losing the focus and
			// this would prevent the completion of the 'rename' or 'create + give initial name' transaction (occurring on focus lost).
			FSlateApplication::Get().ClearKeyboardFocus();

			ClearedFocus = true;
		}
	}
		
	TUniquePtr<FScopedTransaction> Transaction = MakeUnique<FScopedTransaction>(SelectedNodes.Num() > 1 ? LOCTEXT("DuplicateComponents", "Duplicate Components") : LOCTEXT("DuplicateComponent", "Duplicate Component"));

	TArray<FSubobjectDataHandle> DuplicatedHandles;
	System->DuplicateSubobjects(GetObjectContextHandle(), SelectedNodes, /* BpContext = */ nullptr, DuplicatedHandles);
	UpdateTree();

	// If nothing was duplicated, no further action is needed
	if (DuplicatedHandles.Num() <= 0)
	{
		return;
	}

	ClearSelection();

	FSubobjectEditorTreeNodePtrType FirstNode = nullptr;

	// Select all duplicated subobjects
	for (const FSubobjectDataHandle& DataHandle : DuplicatedHandles)
	{
		if (const FSubobjectEditorTreeNodePtrType NewNode = FindSlateNodeForHandle(DataHandle))
		{
			if (!FirstNode)
			{
				FirstNode = NewNode;
			}

			SelectNode(NewNode, true);
		}
	}

	// If only one subobject was duplicated, start a rename operation on it
	if (DuplicatedHandles.Num() == 1)
	{
		OnRenameComponent(MoveTemp(Transaction));
	}
	// Otherwise, scroll the first of the duplicated nodes into view in the tree widget
	else if (FirstNode)
	{
		TreeWidget->RequestScrollIntoView(FirstNode);
		if (ClearedFocus)
		{
			// If focus had to be cleared, put it back on the tree widget so further interactions can be made with the new selection
			FSlateApplication::Get().SetKeyboardFocus(TreeWidget, EFocusCause::SetDirectly);
		}
	}
}

void SSubobjectInstanceEditor::PasteNodes()
{
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		TArray<FSubobjectDataHandle> OutHandles;
		TArray<FSubobjectDataHandle> HandlesToPasteOnto = GetSelectedHandles();
		if (HandlesToPasteOnto.IsEmpty())
		{
			if (FSubobjectEditorTreeNodePtrType RootPtr = GetSceneRootNode())
			{
				if (RootPtr.IsValid())
				{
					HandlesToPasteOnto.Emplace(RootPtr->GetDataHandle());
				}
			}
		}

		System->PasteSubobjects(GetObjectContextHandle(), HandlesToPasteOnto, nullptr, OutHandles);

		if(OutHandles.Num() > 0)
		{
			// We only want the pasted node(s) to be selected
			TreeWidget->ClearSelection();
			UpdateTree();

			for(const FSubobjectDataHandle& Handle : OutHandles)
			{
				if(FSubobjectEditorTreeNodePtrType SlateNode = FindSlateNodeForHandle(Handle))
				{
					TreeWidget->RequestScrollIntoView(SlateNode);
					TreeWidget->SetItemSelection(SlateNode, true);	
				}
			}
		}
	}
}

void SSubobjectInstanceEditor::OnAttachToDropAction(FSubobjectEditorTreeNodePtrType DroppedOn, const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs)
{
	// Ask the subsystem to attach the dropped nodes onto the dropped on node
	check(DroppedOn.IsValid());
	check(DroppedNodePtrs.Num() > 0);
	
	USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
	check(System);
	
	const FScopedTransaction TransactionContext(DroppedNodePtrs.Num() > 1 ? LOCTEXT("AttachComponents", "Attach Components") : LOCTEXT("AttachComponent", "Attach Component"));

	TArray<FSubobjectDataHandle> HandlesToMove;
	Utils::PopulateHandlesArray(DroppedNodePtrs, HandlesToMove);

	FReparentSubobjectParams Params;
	Params.NewParentHandle = DroppedOn->GetDataHandle();
	System->ReparentSubobjects(Params, HandlesToMove);

	check(TreeWidget.IsValid());
	TreeWidget->SetItemExpansion(DroppedOn, true);

	PostDragDropAction(true);
}

void SSubobjectInstanceEditor::OnDetachFromDropAction(const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs)
{
	check(DroppedNodePtrs.Num() > 0);

	const FScopedTransaction TransactionContext(DroppedNodePtrs.Num() > 1 ? LOCTEXT("DetachComponents", "Detach Components") : LOCTEXT("DetachComponent", "Detach Component"));

	TArray<FSubobjectDataHandle> HandlesToMove;
	Utils::PopulateHandlesArray(DroppedNodePtrs, HandlesToMove);

	// Attach the dropped node to the current scene root node
	FSubobjectEditorTreeNodePtrType SceneRootNodePtr = GetSceneRootNode();
	check(SceneRootNodePtr.IsValid());
	
	// Ask the subsystem to reparent this object to the scene root
	if(USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get())
	{
		FReparentSubobjectParams Params;
		Params.NewParentHandle = SceneRootNodePtr->GetDataHandle();
		System->ReparentSubobjects(Params, HandlesToMove);
	}

	PostDragDropAction(true);
}

void SSubobjectInstanceEditor::OnMakeNewRootDropAction(FSubobjectEditorTreeNodePtrType DroppedNodePtr)
{
	// Get the current scene root node
	FSubobjectEditorTreeNodePtrType SceneRootNodePtr = GetSceneRootNode();
	
	// Create a transaction record
	const FScopedTransaction TransactionContext(LOCTEXT("MakeNewSceneRoot", "Make New Scene Root"));

	USubobjectDataSubsystem* Subsystem = USubobjectDataSubsystem::Get();
	const bool bSuccess = Subsystem->MakeNewSceneRoot(
        GetObjectContextHandle(),
         DroppedNodePtr->GetDataHandle(),
         nullptr);
	
	PostDragDropAction(true);
}

void SSubobjectInstanceEditor::PostDragDropAction(bool bRegenerateTreeNodes)
{
	GUnrealEd->ComponentVisManager.ClearActiveComponentVis();

	UpdateTree(bRegenerateTreeNodes);

	if(AActor* ActorInstance = Cast<AActor>(GetObjectContext()))
	{
		ActorInstance->RerunConstructionScripts();
	}
}

TSharedPtr<SWidget> SSubobjectInstanceEditor::BuildSceneRootDropActionMenu(FSubobjectEditorTreeNodePtrType DroppedOntoNodePtr, FSubobjectEditorTreeNodePtrType DroppedNodePtr)
{
	FMenuBuilder MenuBuilder(true, CommandList);

	const FSubobjectData* DroppedNodeData = DroppedNodePtr->GetDataSource();
	const FSubobjectData* DroppedOntoNodeData = DroppedOntoNodePtr->GetDataSource();

	check(DroppedNodeData);
	
	MenuBuilder.BeginSection("SceneRootNodeDropActions", LOCTEXT("SceneRootNodeDropActionContextMenu", "Drop Actions"));
	{
		const FText DroppedVariableNameText = FText::FromName( DroppedNodeData->GetVariableName() );
		
		const FText NodeVariableNameText = FText::FromName( DroppedOntoNodeData->GetVariableName() );

		bool bDroppedInSameBlueprint = true;

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DropActionLabel_AttachToRootNode", "Attach"),
			bDroppedInSameBlueprint 
			? FText::Format( LOCTEXT("DropActionToolTip_AttachToRootNode", "Attach {0} to {1}."), DroppedVariableNameText, NodeVariableNameText )
			: FText::Format( LOCTEXT("DropActionToolTip_AttachToRootNodeFromCopy", "Copy {0} to a new variable and attach it to {1}."), DroppedVariableNameText, NodeVariableNameText ),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSubobjectEditor::OnAttachToDropAction, DroppedOntoNodePtr, DroppedNodePtr),
				FCanExecuteAction()));

		const bool bIsDefaultSceneRoot = DroppedNodeData->IsDefaultSceneRoot();

		FText NewRootNodeText = bIsDefaultSceneRoot
			? FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNodeAndDelete", "Make {0} the new root. The default root will be deleted."), DroppedVariableNameText)
			: FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNode", "Make {0} the new root."), DroppedVariableNameText);

		FText NewRootNodeFromCopyText = bIsDefaultSceneRoot
			? FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNodeFromCopyAndDelete", "Copy {0} to a new variable and make it the new root. The default root will be deleted."), DroppedVariableNameText)
			: FText::Format(LOCTEXT("DropActionToolTip_MakeNewRootNodeFromCopy", "Copy {0} to a new variable and make it the new root."), DroppedVariableNameText);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DropActionLabel_MakeNewRootNode", "Make New Root"),
			bDroppedInSameBlueprint ? NewRootNodeText : NewRootNodeFromCopyText,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSubobjectEditor::OnMakeNewRootDropAction, DroppedNodePtr),
				FCanExecuteAction()));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FSubobjectDataHandle SSubobjectInstanceEditor::AddNewSubobject(const FSubobjectDataHandle& ParentHandle, UClass* NewClass, UObject* AssetOverride, FText& OutFailReason, TUniquePtr<FScopedTransaction> InOngoingTransaction)
{
	FAddNewSubobjectParams Params;
	Params.ParentHandle = ParentHandle;
	Params.NewClass = NewClass;
	Params.AssetOverride = AssetOverride;
	// This is an instance, so the blueprint context is null!
	Params.BlueprintContext = nullptr;
	USubobjectDataSubsystem* System = USubobjectDataSubsystem::Get();
	check(System);
	
	return System->AddNewSubobject(Params,OutFailReason);	
}

void SSubobjectInstanceEditor::PopulateContextMenuImpl(UToolMenu* InMenu, TArray<FSubobjectEditorTreeNodePtrType>& InSelectedItems, bool bIsChildActorSubtreeNodeSelected)
{
	TArray<UActorComponent*> SelectedComponents;
	for(const FSubobjectEditorTreeNodePtrType& SelectedNodePtr : InSelectedItems)
	{
		check(SelectedNodePtr->IsValid());
		// Get the component template associated with the selected node
		const UActorComponent* ComponentTemplate = SelectedNodePtr->GetComponentTemplate();
		if (ComponentTemplate)
		{
			// #TODO_BH Remove this const cast
			SelectedComponents.Add(const_cast<UActorComponent*>(ComponentTemplate));
		}
	}
	
	// Common menu options added for all component types
    FComponentEditorUtils::FillComponentContextMenuOptions(InMenu, SelectedComponents);
}

FMenuBuilder SSubobjectInstanceEditor::CreateMenuBuilder()
{
	FMenuBuilder EditBlueprintMenuBuilder = SSubobjectEditor::CreateMenuBuilder();

	EditBlueprintMenuBuilder.BeginSection(NAME_None, LOCTEXT("EditBlueprintMenu_InstanceHeader", "Instance modifications"));

	EditBlueprintMenuBuilder.AddMenuEntry
	(
		LOCTEXT("PushChangesToBlueprint", "Apply Instance Changes to Blueprint"),
		TAttribute<FText>(this, &SSubobjectInstanceEditor::OnGetApplyChangesToBlueprintTooltip),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SSubobjectInstanceEditor::OnApplyChangesToBlueprint))
	);

	EditBlueprintMenuBuilder.AddMenuEntry
	(
		LOCTEXT("ResetToDefault", "Reset Instance Changes to Blueprint Default"),
		TAttribute<FText>(this, &SSubobjectInstanceEditor::OnGetResetToBlueprintDefaultsTooltip),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SSubobjectInstanceEditor::OnResetToBlueprintDefaults))
	);

	EditBlueprintMenuBuilder.BeginSection(NAME_None, LOCTEXT("EditBlueprintMenu_NewHeader", "Create New"));
	//EditBlueprintMenuBuilder.AddMenuSeparator();

	EditBlueprintMenuBuilder.AddMenuEntry
	(
		LOCTEXT("CreateChildBlueprint", "Create Child Blueprint Class"),
		LOCTEXT("CreateChildBlueprintTooltip", "Creates a Child Blueprint Class based on the current Blueprint, allowing you to create variants easily.  This replaces the current actor instance with a new one based on the new Child Blueprint Class."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SSubobjectInstanceEditor::PromoteToBlueprint))
	);

	return EditBlueprintMenuBuilder;
}

FText SSubobjectInstanceEditor::OnGetApplyChangesToBlueprintTooltip() const
{
	int32 NumChangedProperties = 0;

	AActor* Actor = Cast<AActor>(GetObjectContext());
	UBlueprint* Blueprint = (Actor != nullptr) ? Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy) : nullptr;

	if (Actor != nullptr && Blueprint != nullptr && Actor->GetClass()->ClassGeneratedBy == Blueprint)
	{
		AActor* BlueprintCDO = Actor->GetClass()->GetDefaultObject<AActor>();
		if (BlueprintCDO != nullptr)
		{
			const EditorUtilities::ECopyOptions::Type CopyOptions = (EditorUtilities::ECopyOptions::Type)(EditorUtilities::ECopyOptions::PreviewOnly | EditorUtilities::ECopyOptions::OnlyCopyEditOrInterpProperties | EditorUtilities::ECopyOptions::SkipInstanceOnlyProperties);
			NumChangedProperties += EditorUtilities::CopyActorProperties(Actor, BlueprintCDO, CopyOptions);
		}
		NumChangedProperties += Actor->GetInstanceComponents().Num();
	}

	if (NumChangedProperties == 0)
	{
		return LOCTEXT("DisabledPushToBlueprintDefaults_ToolTip", "Replaces the Blueprint's defaults with any altered property values.");
	}
	else if (NumChangedProperties > 1)
	{
		return FText::Format(LOCTEXT("PushToBlueprintDefaults_ToolTip", "Click to apply {0} changed properties to the Blueprint."), FText::AsNumber(NumChangedProperties));
	}
	else
	{
		return LOCTEXT("PushOneToBlueprintDefaults_ToolTip", "Click to apply 1 changed property to the Blueprint.");
	}
}

void SSubobjectInstanceEditor::OnApplyChangesToBlueprint() const
{
	AActor* Actor = Cast<AActor>(GetObjectContext());
	const UBlueprint* const Blueprint = (Actor != nullptr) ? Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy) : nullptr;

	if (Actor != nullptr && Blueprint != nullptr && Actor->GetClass()->ClassGeneratedBy.Get() == Blueprint)
	{
		const FString ActorLabel = Actor->GetActorLabel();
		int32 NumChangedProperties = FKismetEditorUtilities::ApplyInstanceChangesToBlueprint(Actor);

		// Set up a notification record to indicate success/failure
		FNotificationInfo NotificationInfo(FText::GetEmpty());
		NotificationInfo.FadeInDuration = 1.0f;
		NotificationInfo.FadeOutDuration = 2.0f;
		NotificationInfo.bUseLargeFont = false;
		SNotificationItem::ECompletionState CompletionState;
		if (NumChangedProperties > 0)
		{
			if (NumChangedProperties > 1)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("BlueprintName"), FText::FromName(Blueprint->GetFName()));
				Args.Add(TEXT("NumChangedProperties"), NumChangedProperties);
				Args.Add(TEXT("ActorName"), FText::FromString(ActorLabel));
				NotificationInfo.Text = FText::Format(LOCTEXT("PushToBlueprintDefaults_ApplySuccess", "Updated Blueprint {BlueprintName} ({NumChangedProperties} property changes applied from actor {ActorName})."), Args);
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("BlueprintName"), FText::FromName(Blueprint->GetFName()));
				Args.Add(TEXT("ActorName"), FText::FromString(ActorLabel));
				NotificationInfo.Text = FText::Format(LOCTEXT("PushOneToBlueprintDefaults_ApplySuccess", "Updated Blueprint {BlueprintName} (1 property change applied from actor {ActorName})."), Args);
			}
			CompletionState = SNotificationItem::CS_Success;
		}
		else
		{
			NotificationInfo.Text = LOCTEXT("PushToBlueprintDefaults_ApplyFailed", "No properties were copied");
			CompletionState = SNotificationItem::CS_Fail;
		}

		// Add the notification to the queue
		const TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		Notification->SetCompletionState(CompletionState);
	}
}

void SSubobjectInstanceEditor::OnResetToBlueprintDefaults()
{
	int32 NumChangedProperties = 0;

	AActor* Actor = Cast<AActor>(GetObjectContext());
	UBlueprint* Blueprint = (Actor != nullptr) ? Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy) : nullptr;

	if ((Actor != nullptr) && (Blueprint != nullptr) && (Actor->GetClass()->ClassGeneratedBy == Blueprint))
	{
		const FScopedTransaction Transaction(LOCTEXT("ResetToBlueprintDefaults_Transaction", "Reset to Class Defaults"));

		{
			AActor* BlueprintCDO = Actor->GetClass()->GetDefaultObject<AActor>();
			if (BlueprintCDO != nullptr)
			{
				const EditorUtilities::ECopyOptions::Type CopyOptions = (EditorUtilities::ECopyOptions::Type)(EditorUtilities::ECopyOptions::OnlyCopyEditOrInterpProperties);
				NumChangedProperties = EditorUtilities::CopyActorProperties(BlueprintCDO, Actor, CopyOptions);
			}
			NumChangedProperties += Actor->GetInstanceComponents().Num();
			Actor->ClearInstanceComponents(true);
		}

		// Set up a notification record to indicate success/failure
		FNotificationInfo NotificationInfo(FText::GetEmpty());
		NotificationInfo.FadeInDuration = 1.0f;
		NotificationInfo.FadeOutDuration = 2.0f;
		NotificationInfo.bUseLargeFont = false;
		SNotificationItem::ECompletionState CompletionState;
		if (NumChangedProperties > 0)
		{
			if (NumChangedProperties > 1)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("BlueprintName"), FText::FromName(Blueprint->GetFName()));
				Args.Add(TEXT("NumChangedProperties"), NumChangedProperties);
				Args.Add(TEXT("ActorName"), FText::FromString(Actor->GetActorLabel()));
				NotificationInfo.Text = FText::Format(LOCTEXT("ResetToBlueprintDefaults_ApplySuccess", "Reset {ActorName} ({NumChangedProperties} property changes applied from Blueprint {BlueprintName})."), Args);
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("BlueprintName"), FText::FromName(Blueprint->GetFName()));
				Args.Add(TEXT("ActorName"), FText::FromString(Actor->GetActorLabel()));
				NotificationInfo.Text = FText::Format(LOCTEXT("ResetOneToBlueprintDefaults_ApplySuccess", "Reset {ActorName} (1 property change applied from Blueprint {BlueprintName})."), Args);
			}
			CompletionState = SNotificationItem::CS_Success;
		}
		else
		{
			NotificationInfo.Text = LOCTEXT("ResetToBlueprintDefaults_Failed", "No properties were reset");
			CompletionState = SNotificationItem::CS_Fail;
		}

		UpdateTree();

		// Add the notification to the queue
		const TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		Notification->SetCompletionState(CompletionState);
	}
}

FText SSubobjectInstanceEditor::OnGetResetToBlueprintDefaultsTooltip() const
{
	int32 NumChangedProperties = 0;

	AActor* Actor = Cast<AActor>(GetObjectContext());
	UBlueprint* Blueprint = (Actor != nullptr) ? Cast<UBlueprint>(Actor->GetClass()->ClassGeneratedBy) : nullptr;
	if (Actor != nullptr && Blueprint != nullptr && Actor->GetClass()->ClassGeneratedBy == Blueprint)
	{
		AActor* BlueprintCDO = Actor->GetClass()->GetDefaultObject<AActor>();
		if (BlueprintCDO != nullptr)
		{
			const EditorUtilities::ECopyOptions::Type CopyOptions = (EditorUtilities::ECopyOptions::Type)(EditorUtilities::ECopyOptions::PreviewOnly | EditorUtilities::ECopyOptions::OnlyCopyEditOrInterpProperties);
			NumChangedProperties += EditorUtilities::CopyActorProperties(BlueprintCDO, Actor, CopyOptions);
		}
		NumChangedProperties += Actor->GetInstanceComponents().Num();
	}

	if (NumChangedProperties == 0)
	{
		return LOCTEXT("DisabledResetBlueprintDefaults_ToolTip", "Resets altered properties back to their Blueprint default values.");
	}
	else if (NumChangedProperties > 1)
	{
		return FText::Format(LOCTEXT("ResetToBlueprintDefaults_ToolTip", "Click to reset {0} changed properties to their Blueprint default values."), FText::AsNumber(NumChangedProperties));
	}
	else
	{
		return LOCTEXT("ResetOneToBlueprintDefaults_ToolTip", "Click to reset 1 changed property to its Blueprint default value.");
	}
}

#undef LOCTEXT_NAMESPACE
