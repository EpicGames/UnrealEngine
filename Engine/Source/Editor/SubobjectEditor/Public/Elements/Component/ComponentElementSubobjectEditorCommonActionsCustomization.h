// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Framework/TypedElementSubobjectEditorMixin.h"

#define UE_API SUBOBJECTEDITOR_API

class FComponentElementSubobjectEditorCommonActionsCustomization : public FTypedElementCommonActionsCustomization, public FTypedElementSubobjectEditorMixin
{
	using Super = FTypedElementCommonActionsCustomization;

public:
	UE_API virtual void DuplicateElements(ITypedElementWorldInterface* InWorldInterface, TArrayView<const FTypedElementHandle> InElementHandles, UWorld* InWorld, const FVector& InLocationOffset, TArray<FTypedElementHandle>& OutNewElements) override;
};

#undef UE_API
