// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SSubobjectEditor;

/**
 * Mixin for subobject editor customizations that provides access to the subobject editor widget.
 */
class FTypedElementSubobjectEditorMixin
{
public:
	/**
	 * Get the subobject editor associated with this customization.
	 */
	TSharedPtr<SSubobjectEditor> GetSubobjectEditor() const
	{
		return SubobjectEditorWeak.Pin();
	}

	/**
	 * Set the subobject editor associated with this customization.
	 */
	void SetSubobjectEditor(const TSharedPtr<SSubobjectEditor>& InSubobjectEditor)
	{
		SubobjectEditorWeak = InSubobjectEditor;
	}

private:
	/** The subobject editor associated with this customization. */
	TWeakPtr<SSubobjectEditor> SubobjectEditorWeak;
};
