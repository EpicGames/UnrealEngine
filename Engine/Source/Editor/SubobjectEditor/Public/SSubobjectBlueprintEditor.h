// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSubobjectEditor.h"

#define UE_API SUBOBJECTEDITOR_API

class IClassViewerFilter;

/**
* This is the editor for subobjects within the blueprint editor that
*/
class SSubobjectBlueprintEditor final : public SSubobjectEditor
{
public:
	DECLARE_DELEGATE_OneParam(FOnHighlightPropertyInDetailsView, const class FPropertyPath&);

private:	
	SLATE_BEGIN_ARGS(SSubobjectBlueprintEditor)
        : _ObjectContext(nullptr)
		, _PreviewActor(nullptr)
        , _AllowEditing(true)
		, _HideComponentClassCombo(false)
        , _OnSelectionUpdated()
		, _OnHighlightPropertyInDetailsView()
		{}

		SLATE_ATTRIBUTE(UObject*, ObjectContext)
		SLATE_ATTRIBUTE(AActor*, PreviewActor)
	    SLATE_ATTRIBUTE(bool, AllowEditing)
		SLATE_ATTRIBUTE(bool, HideComponentClassCombo)
	    SLATE_EVENT(FOnSelectionUpdated, OnSelectionUpdated)
	    SLATE_EVENT(FOnItemDoubleClicked, OnItemDoubleClicked)
		SLATE_EVENT(FOnHighlightPropertyInDetailsView, OnHighlightPropertyInDetailsView)
		SLATE_ARGUMENT(TArray<TSharedRef<IClassViewerFilter>>, SubobjectClassListFilters)
		SLATE_ARGUMENT(TSharedPtr<ISCSEditorUICustomization>, SCSEditorUICustomization)
	    
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

protected:

	/** Attribute that provides access to a "preview" Actor context (may not be same as the Actor context that's being edited. */
	TAttribute<AActor*> PreviewActor;
	
	// SSubobjectEditor interface
	virtual bool ShouldModifyBPOnAssetDrop() const override { return true; }
	UE_API virtual void OnDeleteNodes() override;
	UE_API virtual void CopySelectedNodes() override;
	UE_API virtual void OnDuplicateComponent() override;
	UE_API virtual bool CanPasteNodes() const override;
	UE_API virtual void PasteNodes() override;
	UE_API virtual void UpdateTree(bool bRegenerateTreeNodes = true) override;

	UE_API virtual void OnAttachToDropAction(FSubobjectEditorTreeNodePtrType DroppedOn, const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs);
    UE_API virtual void OnDetachFromDropAction(const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs);
    UE_API virtual void OnMakeNewRootDropAction(FSubobjectEditorTreeNodePtrType DroppedNodePtr);
    UE_API virtual void PostDragDropAction(bool bRegenerateTreeNodes);

    /** Builds a context menu pop up for dropping a child node onto the scene root node */
    UE_API virtual TSharedPtr<SWidget> BuildSceneRootDropActionMenu(FSubobjectEditorTreeNodePtrType DroppedOntoNodePtr, FSubobjectEditorTreeNodePtrType DroppedNodePtr);
	virtual bool ClearSelectionOnClick() const override { return true; }
	virtual bool ShowInlineSearchWithButtons() const override { return true; }
	UE_API virtual FSubobjectDataHandle AddNewSubobject(const FSubobjectDataHandle& ParentHandle, UClass* NewClass, UObject* AssetOverride, FText& OutFailReason, TUniquePtr<FScopedTransaction> InOngoingTransaction) override;
	UE_API virtual void PopulateContextMenuImpl(UToolMenu* InMenu, TArray<FSubobjectEditorTreeNodePtrType>& InSelectedItems, bool bIsChildActorSubtreeNodeSelected) override;
	UE_API virtual FSubobjectEditorTreeNodePtrType GetSceneRootNode() const override;
public:
	UE_API virtual FSubobjectEditorTreeNodePtrType FindSlateNodeForObject(const UObject* InObject, bool bIncludeAttachmentComponents = true) const override;
	// End of SSubobjectEditor

public:

	/** Delegate to invoke when the given property should be highlighted in the details view (e.g. diff). */
	FOnHighlightPropertyInDetailsView OnHighlightPropertyInDetailsView;
	
	/**
	* Fills out an events section in ui.
	* @param Menu								the menu to add the events section into
	* @param Blueprint							the active blueprint context being edited
	* @param SelectedClass						the common component class to build the events list from
	* @param CanExecuteActionDelegate			the delegate to query whether or not to execute the UI action
	* @param GetSelectedObjectsDelegate		the delegate to fill the currently select variables / components
	*/
	static UE_API void BuildMenuEventsSection(FMenuBuilder& Menu, UBlueprint* Blueprint, UClass* SelectedClass, FCanExecuteAction CanExecuteActionDelegate, FGetSelectedObjectsDelegate GetSelectedObjectsDelegate);

	/**
	* Highlight a tree node and, optionally, a property with in it
	*
	* @param TreeNodeName		Name of the treenode to be highlighted
	* @param Property	The name of the property to be highlighted in the details view
	*/
	UE_API void HighlightTreeNode(FName TreeNodeName, const class FPropertyPath& Property);
	
protected:

	/**
	* Function to create events for the current selection
	* @param Blueprint						the active blueprint context
	* @param EventName						the event to add
	* @param GetSelectedObjectsDelegate	the delegate to gather information about current selection
	* @param NodeIndex						an index to a specified node to add event for or < 0 for all selected nodes.
	*/
	static UE_API void CreateEventsForSelection(UBlueprint* Blueprint, FName EventName, FGetSelectedObjectsDelegate GetSelectedObjectsDelegate);

	/**
	* Function to construct an event for a node
	* @param Blueprint						the nodes blueprint
	* @param EventName						the event to add
	* @param EventData						the event data structure describing the node
	*/
	static UE_API void ConstructEvent(UBlueprint* Blueprint, const FName EventName, const FComponentEventConstructionData EventData);

	/**
	* Function to view an event for a node
	* @param Blueprint						the nodes blueprint
	* @param EventName						the event to view
	* @param EventData						the event data structure describing the node
	*/
	static UE_API void ViewEvent(UBlueprint* Blueprint, const FName EventName, const FComponentEventConstructionData EventData);

	/** Get the current preview actor for this blueprint editor. */
	AActor* GetActorPreview() const { return PreviewActor.Get(nullptr); }

	TSharedPtr<class SBox> TreeViewOverlay;
};

#undef UE_API
