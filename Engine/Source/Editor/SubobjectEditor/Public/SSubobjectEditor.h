// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "SubobjectDataSubsystem.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Commands/UICommandList.h"
#include "SComponentClassCombo.h"

#include "BlueprintEditor.h"		// FComponentEventConstructionData

#include "ScopedTransaction.h"
#include "DataStorage/Handles.h"

#define UE_API SUBOBJECTEDITOR_API

class SSubobjectEditor;
class SToolTip;
class SExtensionPanel;
class AActor;
class FSubobjectEditorTreeNode;
class SInlineEditableTextBlock;
class ISCSEditorUICustomization;
class UBlueprint;
class UToolMenu;
class UPrimitiveComponent;

// Tree node pointer types
using FSubobjectEditorTreeNodePtrType = TSharedPtr<class FSubobjectEditorTreeNode>;

/////////////////////////////////////////////////////
// FSubobjectEditorTreeNode

/**
 * Wrapper struct that represents access an singe subobject that is used 
 * by slate as a layout for columns/rows should look like
 */
class FSubobjectEditorTreeNode : public TSharedFromThis<FSubobjectEditorTreeNode>
{
public:

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	
	UE_API explicit FSubobjectEditorTreeNode(const FSubobjectDataHandle& DataSource, bool InbIsSeperator = false);

	~FSubobjectEditorTreeNode() = default;

	bool IsSeperator() const { return bIsSeperator; } 

	// If this subobject is an actor component, then return its template. Otherwise it will
	// return null
	UE_API const UActorComponent* GetComponentTemplate() const;

	UE_API const UObject* GetObject(bool bEvenIfPendingKill = false) const;

	// Get a pointer to the subobject data that this slate node is representing
	UE_API FSubobjectData* GetDataSource() const;

	// Get the subobject handle that this slate node is representing
	FSubobjectDataHandle GetDataHandle() const { return DataHandle; }

	/** Get the slate parent of this single tree node */
	FSubobjectEditorTreeNodePtrType GetParent() const { return ParentNodePtr; }

	/** Get the display name of this single tree node */
	UE_API FString GetDisplayString() const;

	UE_API bool IsComponentNode() const;

	// Returns true if this is the root actor node of the tree
	UE_API bool IsRootActorNode() const;

	UE_API bool CanReparent() const;

	UE_API FName GetVariableName() const;

	// Returns true if this subobject data handle is valid
	bool IsValid() const { return bIsSeperator || DataHandle.IsValid(); }

	UE_API bool IsChildSubtreeNode() const;

	UE_API bool IsNativeComponent() const;
	
	// Returns true if this node is attached to the given slate node
	UE_API bool IsAttachedTo(FSubobjectEditorTreeNodePtrType InNodePtr) const;

	/**
	* @return Whether or not this node is a direct child of the given node.
	*/
	UE_API bool IsDirectlyAttachedTo(FSubobjectEditorTreeNodePtrType InNodePtr) const;
	
	UE_API bool CanDelete() const;

	UE_API bool CanRename() const;
	
	/**
	 * @return The set of nodes which are parented to this node (read-only).
	 */
	const TArray<FSubobjectEditorTreeNodePtrType>& GetChildren() const { return Children; }

	// Add the given slate node to our child array and set it's parent to us
	// This had no effect on the actual structure of the subobjects this node represents
	// it is purely visual
	UE_API void AddChild(FSubobjectEditorTreeNodePtrType AttachToPtr);

	// Remove the given slate node from our children array
	UE_API void RemoveChild(FSubobjectEditorTreeNodePtrType InChildNodePtr);

	// Attempts to find the given subobject handle in the slate children on this node. Nullptr if none are found.
	UE_API FSubobjectEditorTreeNodePtrType FindChild(const FSubobjectDataHandle& InHandle);

	/** Query that determines if this item should be filtered out or not */
	UE_API bool IsFlaggedForFiltration() const;

	/** Sets this item's filtration state. Use bUpdateParent to make sure the parent's EFilteredState::ChildMatches flag is properly updated based off the new state */
	UE_API void SetCachedFilterState(bool bMatchesFilter, bool bUpdateParent);

	/** Used to update the EFilteredState::ChildMatches flag for parent nodes, when this item's filtration state has changed */
	UE_API void ApplyFilteredStateToParent();

	/** Updates the EFilteredState::ChildMatches flag, based off of children's current state */
	UE_API void RefreshCachedChildFilterState(bool bUpdateParent);

	/** Refreshes this item's filtration state. Set bRecursive to 'true' to refresh any child nodes as well */
	UE_API bool RefreshFilteredState(const UClass* InFilterType, const TArray<FString>& InFilterTerms, bool bRecursive);

	/** Returns whether the node will match the given type (for filtering) */
	UE_API bool MatchesFilterType(const UClass* InFilterType) const;

	UE_API void SetOngoingCreateTransaction(TUniquePtr<FScopedTransaction> InTransaction);
	UE_API void CloseOngoingCreateTransaction();
	void GetOngoingCreateTransaction(TUniquePtr<FScopedTransaction>& OutPtr) { OutPtr = MoveTemp(OngoingCreateTransaction); }
	bool HasOngoingTransaction() const { return OngoingCreateTransaction.IsValid(); } 

	/** Sets up the delegate for a rename operation */
	void SetRenameRequestedDelegate(FOnRenameRequested InRenameRequested) { RenameRequestedDelegate = InRenameRequested; }
	FOnRenameRequested GetRenameRequestedDelegate() { return RenameRequestedDelegate; }
	
	void ForEachNodeImpl(TFunctionRef<void(FSubobjectEditorTreeNode& Data)> Ftor);
	void DisposeUnsafe();
protected: 

	/** Pointer to the parent of this subobject */
	FSubobjectEditorTreeNodePtrType ParentNodePtr;
	
	// Scope the creation of a node which ends when the initial 'name' is given/accepted by the user, which can be several frames after the node was actually created.
	TUniquePtr<FScopedTransaction> OngoingCreateTransaction;

	/** Handles rename requests */
	FOnRenameRequested RenameRequestedDelegate;
	
	/**
	* Any children that this subobject has in the hierarchy, used to
	* collapsed things within in the tree
	*/
	TArray<FSubobjectEditorTreeNodePtrType> Children;

	/** The data source of this subobject */
	FSubobjectDataHandle DataHandle;

	enum EFilteredState
	{
		FilteredOut		= 0x00,
		MatchesFilter	= (1 << 0),
		ChildMatches	= (1 << 1),

		FilteredInMask = (MatchesFilter | ChildMatches),
		Unknown = 0xFC // ~FilteredInMask
	};
	uint8 FilterFlags;

	/** A flag that indicates that this is a separator slate node and has no valid data */
	uint8 bIsSeperator : 1;
};

/////////////////////////////////////////////////////
// SSubobjectEditorDragDropTree

/** Implements the specific node type and add drag/drop functionality */
class SSubobjectEditorDragDropTree : public STreeView<FSubobjectEditorTreeNodePtrType>
{
public:
	SLATE_BEGIN_ARGS(SSubobjectEditorDragDropTree)
		: _SubobjectEditor(nullptr)
		, _OnGenerateRow()
		, _OnGetChildren()
		, _OnSetExpansionRecursive()
		, _TreeItemsSource(static_cast<TArray<FSubobjectEditorTreeNodePtrType>*>(nullptr))
		, _ItemHeight(16)
		, _OnContextMenuOpening()
		, _OnMouseButtonDoubleClick()
		, _OnSelectionChanged()
		, _SelectionMode(ESelectionMode::Multi)
		, _ClearSelectionOnClick(true)
		, _ExternalScrollbar()
		, _OnTableViewBadState()
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}

		SLATE_ARGUMENT(SSubobjectEditor*, SubobjectEditor)

		SLATE_EVENT(FOnGenerateRow, OnGenerateRow)

		SLATE_EVENT(FOnItemScrolledIntoView, OnItemScrolledIntoView)

		SLATE_EVENT(FOnGetChildren, OnGetChildren)

		SLATE_EVENT(FOnSetExpansionRecursive, OnSetExpansionRecursive)

		SLATE_ARGUMENT(TArray<FSubobjectEditorTreeNodePtrType>*, TreeItemsSource)

		SLATE_ATTRIBUTE_DEPRECATED(float, ItemHeight, 5.5, "The ItemHeight is only used for Tile. See ShouldArrangeAsTiles")

		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

		SLATE_EVENT(FOnMouseButtonDoubleClick, OnMouseButtonDoubleClick)

		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

		SLATE_ATTRIBUTE(ESelectionMode::Type, SelectionMode)

		SLATE_ARGUMENT(TSharedPtr<SHeaderRow>, HeaderRow)

		SLATE_ARGUMENT(bool, ClearSelectionOnClick)

		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalScrollbar)

		SLATE_EVENT(FOnTableViewBadState, OnTableViewBadState)

	SLATE_END_ARGS()

	/** Object construction - mostly defers to the base STreeView */
	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End SWidget interface

private:

	SSubobjectEditor* SubobjectEditor;
};

/////////////////////////////////////////////////////
// SSubobject_RowWidget

/**
* A row widget that represents a single subobject within the tree. 
*/
class SSubobject_RowWidget : public SMultiColumnTableRow<FSubobjectEditorTreeNodePtrType>
{
public:
	SLATE_BEGIN_ARGS(SSubobject_RowWidget) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TWeakPtr<SSubobjectEditor> InEditor, FSubobjectEditorTreeNodePtrType InNodePtr, TSharedPtr<STableViewBase> InOwnerTableView);

	// SMultiColumnTableRow<T> interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:
	virtual ESelectionMode::Type GetSelectionMode() const override;
	// End of SMultiColumnTableRow<T>

	/** Drag-drop handlers */
	void HandleOnDragEnter(const FDragDropEvent& DragDropEvent);
	void HandleOnDragLeave(const FDragDropEvent& DragDropEvent);
	FReply HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	TOptional<EItemDropZone> HandleOnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FSubobjectEditorTreeNodePtrType TargetItem);
	FReply HandleOnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FSubobjectEditorTreeNodePtrType TargetItem);

public:	
	/** Get the asset name of this subobject from the asset brokerage */
	FText GetAssetName() const;

	/** Get the asset path of this subobject from the asset brokerage */
	FText GetAssetPath() const;

	FText GetNameLabel() const;

	/** Check if this asset is visible from the asset brokerage */
	EVisibility GetAssetVisibility() const;

	virtual const FSlateBrush* GetIconBrush() const;

	FSlateColor GetColorTintForIcon() const;

	FSubobjectEditorTreeNodePtrType GetSubobjectPtr() const { return SubobjectPtr; }
	
private:

	static void AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, TSharedRef<SWidget> ValueIcon, const TAttribute<FText>& Value, bool bImportant);

	/** Creates a tooltip for this row */
	TSharedRef<SToolTip> CreateToolTipWidget() const;

	/** Create the tooltip for component subobjects */
	TSharedRef<SToolTip> CreateComponentTooltipWidget(const FSubobjectEditorTreeNodePtrType& InNode) const;
	
	/** Create the tooltip for actor subobjects */
	TSharedRef<SToolTip> CreateActorTooltipWidget(const FSubobjectEditorTreeNodePtrType& InNode) const;
	
	FText GetToolTipText() const;
	
	FText GetActorClassNameText() const;
	FText GetActorSuperClassNameText() const;
	FText GetActorMobilityText() const;
	
	/** Returns a widget that represents the inheritance of this subobject which includes a hyperlink to edit the property */
	TSharedRef<SWidget> GetInheritedLinkWidget();
	
	/** Gets the context of this subobject, such as "(Self)" or "(Instance)" for actors. */
	FText GetObjectContextText() const;

	FString GetDocumentationLink() const;
	FString GetDocumentationExcerptName() const;

	/** Callback used when the user clicks on a blueprint inherited variable */
	void OnEditBlueprintClicked();

	EVisibility GetEditBlueprintVisibility() const;
	EVisibility GetEditNativeCppVisibility() const;
	
	/** Callback used when the user clicks on a native inherited variable */
	void OnEditNativeCppClicked();

	/**
	 * Retrieves tooltip text describing the specified component's mobility.
	 *
	 * @returns An FText object containing a description of the component's mobility
	 */
	FText GetMobilityToolTipText() const;
	
	/**
	 * Retrieves an image brush signifying the specified component's mobility (could sometimes be NULL).
	 *
	 * @returns A pointer to the FSlateBrush to use (NULL for Static and Non-SceneComponents)
	 */
	FSlateBrush const* GetMobilityIconImage() const;

	/**
	* Retrieves tooltip text describing where the component was first introduced (for inherited components).
	* 
	* @returns An FText object containing a description of when the component was first introduced
	*/
	FText GetIntroducedInToolTipText() const;
	
	/**
	* Retrieves tooltip text describing how the component was introduced
	* 
	* @returns An FText object containing a description of when the component was first introduced
	*/
	FText GetComponentAddSourceToolTipText() const;

	/**
	* Retrieves a tooltip text describing if the component is marked Editor only or not
	*
	* @returns An FText object containing a description of if the component is marked Editor only or not
	*/
	FText GetComponentEditorOnlyTooltipText() const;
	
	/**
	* Retrieves tooltip text for the specified Native Component's underlying Name
	*
	* @returns An FText object containing the Component's Name
	*/
	FText GetNativeComponentNameToolTipText() const;

	FText GetActorDisplayText() const;

	/** Commits the new name of the component */
	void OnNameTextCommit(const FText& InNewName, ETextCommit::Type InTextCommit);

	/** Verifies the name of the component when changing it */
	bool OnNameTextVerifyChanged(const FText& InNewText, FText& OutErrorMessage);

	bool IsReadOnly() const;

	TWeakPtr<SSubobjectEditor> SubobjectEditor;
	TSharedPtr<SInlineEditableTextBlock> InlineWidget;

	FSubobjectEditorTreeNodePtrType SubobjectPtr;
};

/////////////////////////////////////////////////////
// SSubobjectEditor

/**
* The base class viewer for subobject editing in Slate. This displays
* all subobjects of a given actor or blueprint. A subobject can be a native
* component on an actor or inherited components from a native or blueprint parent.
*/
class SSubobjectEditor : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnSelectionUpdated, const TArray<FSubobjectEditorTreeNodePtrType>&);
	DECLARE_DELEGATE_OneParam(FOnItemDoubleClicked, const FSubobjectEditorTreeNodePtrType);
	
protected:
	
	// Do not allow public construction of this widget!
	SSubobjectEditor() = default;

	UE_API virtual ~SSubobjectEditor();

	/** Delegate to invoke on selection update. */
	FOnSelectionUpdated OnSelectionUpdated;

	/** Delegate to invoke when an item in the tree is double clicked. */
	FOnItemDoubleClicked OnItemDoubleClicked;

	/** Create and parent the Editor Scope Tree row.  Call once from derived Construct(). */
	UE_API void InitScopeRow(FStringView Label);

	/** Context row owned by this subobject editor for Editor Scope Tree.
	 *  Created by InitScopeRow(), parented to the current context at that time.
	 *  Pushed onto TLS in event handlers so property visibility checks see the correct editing scope. */
	UE::Editor::DataStorage::RowHandle ScopeRow = UE::Editor::DataStorage::InvalidRowHandle;

	/** Attribute that provides access to the Object context for which we are viewing/editing. */
	TAttribute<UObject*> ObjectContext;

	/** Attribute to indicate whether or not editing is allowed. */
	TAttribute<bool> AllowEditing;

	/** Attribute to indicate whether or not the "Add Component" button is visible. If true, new components cannot be added to the Blueprint. */
	TAttribute<bool> HideComponentClassCombo;

	/** Attribute to limit visible nodes to a particular component type when filtering the tree view. */
	TAttribute<TSubclassOf<UActorComponent>> ComponentTypeFilter;

	/////////////////////////////////////////////////////////
	// Widget Callbacks
protected:

	UE_API TSharedRef<ITableRow> MakeTableRowWidget(FSubobjectEditorTreeNodePtrType InNodePtr, const TSharedRef<STableViewBase>& OwnerTable);

	/** @return The visibility of the components tree */
	UE_API EVisibility GetComponentsTreeVisibility() const;

	/** Used by tree control - get children for a specified subobject node */
	UE_API void OnGetChildrenForTree(FSubobjectEditorTreeNodePtrType InNodePtr, TArray<FSubobjectEditorTreeNodePtrType>& OutChildren);

	/** Update any associated selection (e.g. details view) from the passed in nodes */
	UE_API void UpdateSelectionFromNodes(const TArray<FSubobjectEditorTreeNodePtrType>& SelectedNodes);

	/** Called when selection in the tree changes */
	UE_API void OnTreeSelectionChanged(FSubobjectEditorTreeNodePtrType InSelectedNodePtr, ESelectInfo::Type SelectInfo);

	/** Callback when a component item is double clicked. */
	UE_API void HandleItemDoubleClicked(FSubobjectEditorTreeNodePtrType InItem);

	UE_API void OnFindReferences(bool bSearchAllBlueprints, const EGetFindReferenceSearchStringFlags Flags);

	/** @return The visibility of the components filter box */
	UE_API EVisibility GetComponentsFilterBoxVisibility() const;

	/** Recursively updates the filtered state for each component item */
	UE_API void OnFilterTextChanged(const FText& InFilterText);

	/** Callback when a component item is scrolled into view */
	UE_API void OnItemScrolledIntoView(FSubobjectEditorTreeNodePtrType InItem, const TSharedPtr<ITableRow>& InWidget);

	/** SWidget interface */
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	
public:
	UE_API void SetSelectionSet(UTypedElementSelectionSet* InSelectionSet);
	
	UE_API UTypedElementSelectionSet* GetSelectionSet() const;

	/** Select the given tree node */
	UE_API void SelectNode(FSubobjectEditorTreeNodePtrType InNodeToSelect, bool bIsCntrlDown);

	/** Select the given tree node if there is on that matches the given handle */
	UE_API void SelectNodeFromHandle(const FSubobjectDataHandle& InHandle, bool bIsCntrlDown);
	
	/** Update the contents of the Subobject Tree based on the current context */
	UE_API virtual void UpdateTree(bool bRegenerateTreeNodes = true);

	/** Dumps out the tree view contents to the log (used to assist with debugging widget hierarchy issues) */
	UE_API void DumpTree();
	
	/** Forces the details panel to refresh on the same objects */
	UE_API void RefreshSelectionDetails();

	/** Clears the current selection */
	UE_API void ClearSelection();

	/** 
	 * Called before a potential change of subobject editor target, so that customizations can be
	 *  given a chance to gather context they might need for a subsequent SelectDefaultSubobject call.
	 */
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	UE_API void PrePotentialTargetChange();

	/** Select the default subobject, which is typically the root, but may be adjusted by a customization*/
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	UE_API void SelectDefaultSubobject();

	/** Select the root of the tree */
	UE_API void SelectRoot();

	/** Get the tint color that any subobject editor icons should use. The default is the foreground color. */
	UE_API virtual FSlateColor GetColorTintForIcon(FSubobjectEditorTreeNodePtrType Node) const;
	
	/** Get the currently selected nodes from the tree sorted in order from parent to child */
	UE_API TArray<FSubobjectEditorTreeNodePtrType> GetSelectedNodes() const;

	/** Returns the number of currently selected nodes in the tree */
	int32 GetNumSelectedNodes() const { return TreeWidget->GetSelectedItems().Num(); }

	/** Get the currently selected handles from the tree sorted in order from parent to child */
	UE_API TArray<FSubobjectDataHandle> GetSelectedHandles() const;

	UE_API virtual FSubobjectEditorTreeNodePtrType GetSceneRootNode() const;

	/** 
	 * Returns whether the editor wants to be visible, which can be optionally used by the containing
	 *  widget. If not overriden by a customization, this will be Collapsed or Visible depending on 
	 *  whether ObjectContext was null or not, respectively, but a customization can drive this by
	 *  some other state.
	 */
	UE_API EVisibility GetDesiredVisibility() const;
	
	/** Try to handle a drag-drop operation */
	UE_API FReply TryHandleAssetDragDropOperation(const FDragDropEvent& DragDropEvent);
	
	/** Handle drag-drop operation on a tree item */
	UE_API FReply TryHandleAssetDragDropOperation(const FDragDropEvent& InDragDropEvent, FSubobjectEditorTreeNodePtrType InTargetItem);
	
	/** Sets UI customizations of this SCSEditor. */
	UE_API void SetUICustomization(TSharedPtr<ISCSEditorUICustomization> InUICustomization);
	
	/** Internally linked function to allow the tree widgets to access customization functions */
	const ISCSEditorUICustomization* GetUICustomization() const;

	UE_API virtual bool IsEditingAllowed() const;
	
	/** Return the button widgets that can add components or create/edit blueprints */
	UE_API TSharedPtr<SWidget> GetToolButtonsBox() const;
	
	TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }

	TSharedPtr<SSubobjectEditorDragDropTree> GetDragDropTree() const { return TreeWidget; }

	// Drag/drop operations
	/** Handler for attaching a single node to this node */
	void OnAttachToDropAction(FSubobjectEditorTreeNodePtrType DroppedOn, FSubobjectEditorTreeNodePtrType DroppedNodePtr)
	{
		TArray<FSubobjectEditorTreeNodePtrType> DroppedNodePtrs;
		DroppedNodePtrs.Add(DroppedNodePtr);
		OnAttachToDropAction(DroppedOn, DroppedNodePtrs);
	}
	
	virtual void OnAttachToDropAction(FSubobjectEditorTreeNodePtrType DroppedOn, const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs) = 0;
	virtual void OnDetachFromDropAction(const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs) = 0;
	virtual void OnMakeNewRootDropAction(FSubobjectEditorTreeNodePtrType DroppedNodePtr) = 0;
	virtual void PostDragDropAction(bool bRegenerateTreeNodes) = 0;

	/** Builds a context menu popup for dropping a child node onto the scene root node */
	virtual TSharedPtr<SWidget> BuildSceneRootDropActionMenu(FSubobjectEditorTreeNodePtrType DroppedOntoNodePtr, FSubobjectEditorTreeNodePtrType DroppedNodePtr) = 0;
	
	virtual bool CanMakeNewRootOnDrag(UBlueprint* DraggedFromBlueprint) const { return false; }

	/** Provides access to the Blueprint context that's being edited */
	UE_API UBlueprint* GetBlueprint() const;

	// Attempt to find an existing slate node that matches the given handle
	UE_API virtual FSubobjectEditorTreeNodePtrType FindSlateNodeForObject(const UObject* InObject, bool bIncludeAttachmentComponents = true) const;
	
	/** Returns true if the specified component is currently selected */
    UE_API bool IsComponentSelected(const UPrimitiveComponent* PrimComponent) const;

    /** Assigns a selection override delegate to the specified component */
    UE_API void SetSelectionOverride(UPrimitiveComponent* PrimComponent) const;

	/**
	* Fills the supplied array with the currently selected objects
	* @param OutSelectedItems The array to fill.
	*/
	UE_API void GetSelectedItemsForContextMenu(TArray<FComponentEventConstructionData>& OutSelectedItems) const;

	/** Return an array of the current slate node hierarchy in the tree */
	const TArray<FSubobjectEditorTreeNodePtrType>& GetRootNodes() const { return RootNodes; };

	// Attempt to find an existing slate node that matches the given handle
	UE_API FSubobjectEditorTreeNodePtrType FindSlateNodeForHandle(const FSubobjectDataHandle& Handle, FSubobjectEditorTreeNodePtrType InStartNodePtr = FSubobjectEditorTreeNodePtrType()) const;

	// Attempt to find an existing slate node that has a given variable name
	UE_API FSubobjectEditorTreeNodePtrType FindSlateNodeForVariableName(FName InVariableName) const;

	/** Pointer to the current object that is represented by the subobject editor */
	UE_API UObject* GetObjectContext() const;

	/** SubobjectHandle of the current object that is represented by the subobject editor */
	UE_API FSubobjectDataHandle GetObjectContextHandle() const;

	/** Get the SubobjectData for the current context represented by the subobject editor
	 * This is a newer API than GetObjectContext + GetObjectContextHandle.  It is introduced to allow
	 * access to non-UObject based data and for cases where SubobjectData is a proxy for multiple objects
	 * in bulk-edit cases.
	 */
	UE_EXPERIMENTAL(5.8, "This function is experimental for now.")
	UE_API FSubobjectData* GetDataContextHandle() const;

	/** Refresh the type list presented by the add component button when clicked */
	UE_API void RefreshComponentTypesList();
	
protected:

	/** Restore the previous selection state when updating the tree */
	UE_API virtual void RestoreSelectionState(TArray<FSubobjectEditorTreeNodePtrType>& SelectedTreeNodes, bool bFallBackToVariableName = true);

	/** If true, then the blueprint should be modified on TryHandleAssetDragDropOperation */
	virtual bool ShouldModifyBPOnAssetDrop() const { return false; }
	
	UE_API bool CanCutNodes() const;
	UE_API void CutSelectedNodes();

	UE_API bool CanCopyNodes() const;
	virtual void CopySelectedNodes() = 0;

	/** Pastes previously copied node(s) */
	UE_API virtual bool CanPasteNodes() const;
	virtual void PasteNodes() = 0;

	/** Returns true if keyboard focus should be cleared in OnDuplicateComponent if the widget is currently focused */
	UE_API static bool ShouldClearFocusOnDuplicate(TSharedPtr<SWidget> InCurrentlyFocusedWidget);
	
public:
	/** Removes existing selected component nodes from the SCS */
	UE_API virtual bool CanDeleteNodes() const;
	virtual void OnDeleteNodes() = 0;
	
protected:
	friend class FComponentElementSubobjectEditorCommonActionsCustomization;

	UE_API bool CanDuplicateComponent() const;
	virtual void OnDuplicateComponent() = 0;

	/** Checks to see if renaming is allowed on the selected component */
	UE_API bool CanRenameComponent() const;
	UE_API void OnRenameComponent();
	UE_API void OnRenameComponent(TUniquePtr<FScopedTransaction> InComponentCreateTransaction);
	
	/** Called at the end of each frame. */
	UE_API void OnPostTick(float);
	
	/////////////////////////////////////////////////////////
	// Widgets

	/** Root set of tree that represents all subobjects for the current context */
	TArray<FSubobjectEditorTreeNodePtrType> RootNodes;

	/** Tree widget */
	TSharedPtr<SSubobjectEditorDragDropTree> TreeWidget;

	/** Command list for handling actions in the editor. */
	TSharedPtr<FUICommandList> CommandList;

	/** The filter box that handles filtering for the tree. */
	TSharedPtr<SSearchBox> FilterBox;

	/** The tools buttons box */
	TSharedPtr<SHorizontalBox> ButtonBox;

	/** The add component button / type selector */
	TSharedPtr<SComponentClassCombo> ComponentClassCombo;

	/** Gate to prevent changing the selection while selection change is being broadcast. */
	bool bUpdatingSelection;

	/** Controls whether or not to allow calls to UpdateTree() */
	bool bAllowTreeUpdates;

	/** SCSEditor UI customizations */
	TSharedPtr<ISCSEditorUICustomization> UICustomization;

	/** SCSEditor UI extension */
	TSharedPtr<SExtensionPanel> ExtensionPanel;

	/** Scope the creation of a component which ends when the initial component 'name' is given/accepted by the user, which can be several frames after the component was actually created. */
	TUniquePtr<FScopedTransaction> DeferredOngoingCreateTransaction;

	/** Used to unregister from the post tick event. */
	FDelegateHandle PostTickHandle;

	/** Name of a node that has been requested to be renamed */
	FSubobjectDataHandle DeferredRenameRequest;

	/**
	 * The handle to the current root context. If this is different in between
	 * UpdateTree calls, then we know the context has changed and we should clean up
	 * the subobject memory layout. 
	 */
	FSubobjectDataHandle CachedRootHandle;

	UE_API virtual FMenuBuilder CreateMenuBuilder();

	/** Constructs the slate drag/drop tree for this subobject editor */
	UE_API virtual void ConstructTreeWidget();

	/** Creates a list of commands */
	UE_API virtual void CreateCommandList();

	/** Recursively visits the given node + its children and invokes the given function for each. */
	UE_API void DepthFirstTraversal(const FSubobjectEditorTreeNodePtrType& InNodePtr, TSet<FSubobjectEditorTreeNodePtrType>& OutVisitedNodes, const TFunctionRef<void(const FSubobjectEditorTreeNodePtrType&)> InFunction) const;

	/** Returns the set of expandable nodes that are currently collapsed in the UI */
	UE_API void GetCollapsedNodes(const FSubobjectEditorTreeNodePtrType& InNodePtr, TSet<FSubobjectEditorTreeNodePtrType>& OutCollapsedNodes) const;

	// Attempt to find an existing slate node that matches the given handle
	static UE_API FSubobjectEditorTreeNodePtrType FindOrCreateSlateNodeForHandle(const FSubobjectDataHandle& Handle, TMap<FSubobjectDataHandle, FSubobjectEditorTreeNodePtrType>& ExistingNodes);

	/**
	 * Set the expansion state of a node
	 *
	 * @param InNodeToChange	The node to be expanded/collapsed
	 * @param bIsExpanded		True to expand the node, false to collapse it
	 */
	UE_API void SetNodeExpansionState(FSubobjectEditorTreeNodePtrType InNodeToChange, const bool bIsExpanded);
	
	/** Handler for recursively expanding/collapsing items */
	UE_API void SetItemExpansionRecursive(FSubobjectEditorTreeNodePtrType Model, bool bInExpansionState);

	/** Registers context menu by name for later access */
	UE_API void RegisterContextMenu();

	/** Called to display context menu when right clicking on the widget */
	UE_API TSharedPtr<SWidget> CreateContextMenu();

	/** Populate context menu on the fly */
	UE_API void PopulateContextMenu(UToolMenu* InMenu);

	/** Populate the context menu with implementation specific details */
	virtual void PopulateContextMenuImpl(UToolMenu* InMenu, TArray<FSubobjectEditorTreeNodePtrType>& InSelectedItems, bool bIsChildActorSubtreeNodeSelected) = 0;
	
	/** @return Type of component to filter the tree view with or nullptr if there's no filter. */
	UE_API TSubclassOf<UActorComponent> GetComponentTypeFilterToApply() const;

	/**
	 * Compares the filter bar's text with the item's component name. Use
	 * bRecursive to refresh the state of child nodes as well. Returns true if
	 * the node is set to be filtered out
	 */
	UE_API bool RefreshFilteredState(FSubobjectEditorTreeNodePtrType TreeNode, bool bRecursive);

	/** Callback for the action trees to get the filter text */
	UE_API FText GetFilterText() const;

	/** Converts the current actor instance to a blueprint */
	UE_API void PromoteToBlueprint() const;

	/** Called when the promote to blueprint button is clicked */
	UE_API FReply OnPromoteToBlueprintClicked();

	/** Opens the blueprint editor for the blueprint being viewed by the scseditor */
	UE_API void OnOpenBlueprintEditor(bool bForceCodeEditing) const;

	/**
	 * Spawns a new SWindow giving the user options for creating a new C++ component class.
	 * The user will be prompted to pick a new subclass name and code will be recompiled
	 *
	 * @return The new class that was created
	 */
	UE_API UClass* SpawnCreateNewCppComponentWindow(TSubclassOf<UActorComponent> ComponentClass);

	/**
	 * Spawns a new SWindow giving the user options for creating a new blueprint component class.
	 * The user will be prompted to pick a new subclass name and a blueprint asset will be created
	 *
	 * @return The new class that was created
	 */
	UE_API UClass* SpawnCreateNewBPComponentWindow(TSubclassOf<UActorComponent> ComponentClass);

	/** Add a component from the selection in the combo box */
	UE_API FSubobjectDataHandle PerformComboAddClass(TSubclassOf<UActorComponent> ComponentClass, EComponentCreateAction::Type ComponentCreateAction, UObject* AssetOverride);

	/** @return The visibility of the promote to blueprint button (only visible with an actor instance that is not created from a blueprint)*/
	UE_API virtual EVisibility GetPromoteToBlueprintButtonVisibility() const;

	/** @return The visibility of the Edit Blueprint button (only visible with an actor instance that is created from a blueprint)*/
	UE_API virtual EVisibility GetEditBlueprintButtonVisibility() const;

	/** @return The visibility of the Add Component combo button */
	UE_API virtual EVisibility GetComponentClassComboButtonVisibility() const;

	/** If true, then the tree widget will clear selection on click. */
	virtual bool ClearSelectionOnClick() const { return false; }
	
	/** Indicates whether or not the search bar and inline buttons are visible. */
	virtual bool ShowInlineSearchWithButtons() const { return false; }

	/** Add a new subobject to the given parent via the subobject data subsystem */
	virtual FSubobjectDataHandle AddNewSubobject(const FSubobjectDataHandle& ParentHandle, UClass* NewClass, UObject* AssetOverride, FText& OutFailReason, TUniquePtr<FScopedTransaction> InOngoingTransaction) = 0;

	///////////////////////////////////////
	// Utils
	struct Utils
	{
		/** Populate an array of subobject handles based on an array of the slate node ptr */
		static void PopulateHandlesArray(const TArray<FSubobjectEditorTreeNodePtrType>& SlateNodePtrs, TArray<FSubobjectDataHandle>& OutHandles);
	};

	void ForEachNode(TFunctionRef<void(FSubobjectEditorTreeNode& Data)> Ftor);
private:
	TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet;
};

#undef UE_API
