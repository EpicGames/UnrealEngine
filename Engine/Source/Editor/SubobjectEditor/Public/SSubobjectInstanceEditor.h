// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSubobjectEditor.h"

#define UE_API SUBOBJECTEDITOR_API

/**
* This is the editor for subobjects within the level editor that
* works mainly with component and actor instances. 
*/
class SSubobjectInstanceEditor final : public SSubobjectEditor
{
private:	
	SLATE_BEGIN_ARGS(SSubobjectInstanceEditor)
        : _ObjectContext(nullptr)
        , _AllowEditing(true)
        , _OnSelectionUpdated()
		{}

		SLATE_ATTRIBUTE(UObject*, ObjectContext)
	    SLATE_ATTRIBUTE(bool, AllowEditing)
	    SLATE_EVENT(FOnSelectionUpdated, OnSelectionUpdated)
	    SLATE_EVENT(FOnItemDoubleClicked, OnItemDoubleClicked)
		SLATE_ARGUMENT(TSharedPtr<ISCSEditorUICustomization>, SCSEditorUICustomization)
	
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

protected:
	
	// SSubobjectEditor interface
	UE_API virtual void OnDeleteNodes() override;
	UE_API virtual void CopySelectedNodes() override;
	UE_API virtual void OnDuplicateComponent() override;
	UE_API virtual void PasteNodes() override;
	
	UE_API virtual void OnAttachToDropAction(FSubobjectEditorTreeNodePtrType DroppedOn, const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs) override;
    UE_API virtual void OnDetachFromDropAction(const TArray<FSubobjectEditorTreeNodePtrType>& DroppedNodePtrs) override;
    UE_API virtual void OnMakeNewRootDropAction(FSubobjectEditorTreeNodePtrType DroppedNodePtr) override;
    UE_API virtual void PostDragDropAction(bool bRegenerateTreeNodes) override;

	UE_API virtual void UpdateTree(bool bRegenerateTreeNodes = true) override;

    /** Builds a context menu popup for dropping a child node onto the scene root node */
    UE_API virtual TSharedPtr<SWidget> BuildSceneRootDropActionMenu(FSubobjectEditorTreeNodePtrType DroppedOntoNodePtr, FSubobjectEditorTreeNodePtrType DroppedNodePtr) override;
	UE_API virtual FSubobjectDataHandle AddNewSubobject(const FSubobjectDataHandle& ParentHandle, UClass* NewClass, UObject* AssetOverride, FText& OutFailReason, TUniquePtr<FScopedTransaction> InOngoingTransaction) override;
	UE_API virtual void PopulateContextMenuImpl(UToolMenu* InMenu, TArray<FSubobjectEditorTreeNodePtrType>& InSelectedItems, bool bIsChildActorSubtreeNodeSelected) override;
	UE_API virtual FMenuBuilder CreateMenuBuilder();
	// End of SSubobjectEditor

public:
private:

	/** @return the tooltip describing how many properties will be applied to the blueprint */
	UE_API FText OnGetApplyChangesToBlueprintTooltip() const;

	/** Propagates instance changes to the blueprint */
	UE_API void OnApplyChangesToBlueprint() const;

	/** Resets instance changes to the blueprint default */
	UE_API void OnResetToBlueprintDefaults();

	/** @return the tooltip describing how many properties will be reset to the blueprint default*/
	UE_API FText OnGetResetToBlueprintDefaultsTooltip() const;

	TSharedPtr<class SBox> TreeViewOverlay;

};

#undef UE_API
