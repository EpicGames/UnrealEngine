// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SubobjectEditorMenuContext.generated.h"

#define UE_API SUBOBJECTEDITOR_API

class SSubobjectEditor;
struct FFrame;

UCLASS(MinimalAPI)
class USubobjectEditorMenuContext : public UObject
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category="Tool Menus")
	UE_API TArray<UObject*> GetSelectedObjects() const;

	TWeakPtr<SSubobjectEditor> SubobjectEditor;
	
	bool bOnlyShowPasteOption;
};

#undef UE_API
