// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SubobjectEditor : ModuleRules
{
	public SubobjectEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"SubobjectDataInterface",
				"Kismet",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"InputCore",
				"KismetWidgets",
				"EditorWidgets",
				"GameProjectGeneration",
				"EditorFramework",
				"GraphEditor",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementFramework"
			}
		);
		
		if(Target.bWithLiveCoding)
		{
			PrivateIncludePathModuleNames.Add("LiveCoding");
		}
	}
}
